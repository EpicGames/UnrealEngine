// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#if defined(UBA_COORDINATOR_HORDE_DLL)

#include "DesktopPlatformModule.h"
#include "HAL/Platform.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HttpManager.h"
#include "HttpModule.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/OutputDeviceConsole.h"
#include "Misc/OutputDeviceRedirector.h"
#include "SocketSubsystem.h"
#include "UbaCoordinator.h"
#include "UbaHordeAgentManager.h"

#if PLATFORM_WINDOWS
#define UBA_EXPORT __declspec(dllexport) 
#else
#define UBA_EXPORT __attribute__ ((visibility("default")))
#endif

TCHAR GInternalProjectName[64] = TEXT( "UbaCoordinatorHorde" );
const TCHAR *GForeignEngineDir = nullptr;

class CoordinatorHorde : public uba::Coordinator
{
public:
	CoordinatorHorde(const FString& workDir, const FString& binariesDir) : m_manager(workDir, binariesDir)
	{
	}

	virtual ~CoordinatorHorde()
	{
	}

	virtual void SetAddClientCallback(AddClientCallback callback, void* userData) override
	{
		m_manager.SetAddClientCallback(callback, userData);
	}

	virtual void SetTargetCoreCount(uba::u32 count) override
	{
		m_manager.SetTargetCoreCount(count);
	}

#if !PLATFORM_WINDOWS
	static void StatusCallback(void* UserData, const TCHAR* Status)
	{
		CoordinatorHorde* CH = (CoordinatorHorde*)UserData;
		FTCHARToUTF8 Converter(Status);
		CH->StatusCallbackFunc(CH->StatusUserData, Converter.Get());
	}
#endif

	virtual void SetUpdateStatusCallback(UpdateStatusCallback* callback, void* userData) override
	{
#if PLATFORM_WINDOWS
		m_manager.SetUpdateStatusCallback(callback, userData);
#else
		m_manager.SetUpdateStatusCallback(StatusCallback, this);
		StatusUserData = userData;
		StatusCallbackFunc = callback;
#endif
	}

	virtual bool RequestCacheServer(uba::tchar* outEndpoint, uba::u32 outEndpointCapacity, uba::Guid& outSessionKey, bool& outWriteAccess) override
	{
		FString Endpoint;
		FString SessionKey;
		bool bWriteAccess;
		if (!m_manager.RequestCacheServer(Endpoint, SessionKey, bWriteAccess))
			return false;
		FString::ToHexBlob(SessionKey, (uint8*)&outSessionKey, sizeof(uba::Guid));
		outWriteAccess = bWriteAccess;
#if PLATFORM_WINDOWS
		const uba::tchar* Mem = *Endpoint;
		uint64 ToCopy = Endpoint.Len() + 1;
#else
		FTCHARToUTF8 Converter(Endpoint);
		const uba::tchar* Mem = Converter.Get();
		uint64 ToCopy = Converter.Length() + 1;
#endif
		if (ToCopy > outEndpointCapacity)
			return false;
		memcpy(outEndpoint, Mem, ToCopy * sizeof(uba::tchar));
		return true;
	}

#if !PLATFORM_WINDOWS
	void* StatusUserData = nullptr;
	UpdateStatusCallback* StatusCallbackFunc = nullptr;
#endif

	FUbaHordeAgentManager m_manager;
};


extern "C"
{
	UBA_EXPORT uba::Coordinator* UbaCreateCoordinator(const uba::CoordinatorCreateInfo& info)
	{
		FCommandLine::Set(TEXT(""));
		GWarn = FPlatformApplicationMisc::GetFeedbackContext();

		FConfigCacheIni::InitializeConfigSystem();

		if (info.logging)
		{
			static auto c = TUniquePtr<FOutputDeviceConsole>(FPlatformApplicationMisc::CreateConsoleOutputDevice());
			GLogConsole = c.Get();
			GLogConsole->Show(true);
			GLog->SetCurrentThreadAsPrimaryThread();
			GLog->TryStartDedicatedPrimaryThread();
			GLog->AddOutputDevice(GLogConsole);
		}

		GGameThreadId = FPlatformTLS::GetCurrentThreadId();
		GIsGameThreadIdInitialized = true;

		// Since we are not setting CWD we need to manually call these systems from this thread (if not called from game thread LoadModule uses CWD which is not set)
		ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
		FHttpModule::Get();
		FDesktopPlatformModule::TryGet();

		return new CoordinatorHorde(info.workDir, info.binariesDir);
	}

	UBA_EXPORT void UbaDestroyCoordinator(uba::Coordinator* coordinator)
	{
		delete (CoordinatorHorde*)coordinator;
		if (GLog)
			GLog->SetCurrentThreadAsPrimaryThread();
		FTextKey::TearDown(); // This is for clean shutdown with tsan
		FHttpModule::Get().GetHttpManager().Shutdown();
	}
}

#else

class FUbaCoordinatorHordeModule : public IModuleInterface
{
	UBACOORDINATORHORDE_API FUbaCoordinatorHordeModule& Get()
	{
		return FModuleManager::LoadModuleChecked<FUbaCoordinatorHordeModule>(TEXT("UbaCoordinatorHorde"));
	}
};

IMPLEMENT_MODULE(FUbaCoordinatorHordeModule, UbaCoordinatorHorde);

#endif
