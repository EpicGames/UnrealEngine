// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaHordeAgent.h"
#include "UbaHordeConfig.h"

#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "Compute/AgentMessage.h"
#include "Compute/ComputeSocket.h"
#include "Compute/ComputeTransport.h"
#include "UbaHordeComputeTransport.h"
#include "UbaHordeComputeTransportAES.h"

DEFINE_LOG_CATEGORY(LogUbaHordeAgent);

FUbaHordeAgent::FUbaHordeAgent(const FHordeRemoteMachineInfo& InMachineInfo)
	: bIsValid(false)
	, bHasErrors(false)
	, MachineInfo(InMachineInfo)
{
	// Create the compute transport object and directly send over the nonce.
	TUniquePtr<FComputeTransport> ComputeTransport = MakeUnique<FUbaHordeComputeTransport>(MachineInfo, bHasErrors);
	if (!ComputeTransport->IsValid())
	{
		return;
	}

	// Send initial nonce to establish connection
	ComputeTransport->Send(MachineInfo.Nonce, sizeof(MachineInfo.Nonce));

	// If encryption protocol is requested, wrap initial transport layer around an encryption layer *after* initial nonce was sent unencrypted
	if (InMachineInfo.Encryption == EUbaHordeEncryption::AES)
	{
		ComputeTransport = MakeUnique<FUbaHordeComputeTransportAES>(MachineInfo, MoveTemp(ComputeTransport));
		if (!ComputeTransport->IsValid())
		{
			return;
		}
	}

	// Create the compute socket and initialize a compute channel with the recv/send buffers.
	// This will allow us to create the agent channel object which will allow us to communicate
	// with the agent directly.
	bool Anonymous = true; // There is no point wasting resources creating named events and file mappings if we use sockets for transport
	HordeComputeSocket = CreateComputeSocket(MoveTemp(ComputeTransport), EComputeSocketEndpoint::Remote);
	TSharedPtr<FComputeChannel> ComputeChannel = HordeComputeSocket->CreateChannel(0, Anonymous);
	TSharedPtr<FComputeChannel> SecondComputeChannel = HordeComputeSocket->CreateChannel(100, Anonymous);

	if (!ComputeChannel.IsValid() || !SecondComputeChannel.IsValid())
	{
		return;
	}

	// Now this is what is going to handle the ComputeChannel.
	AgentChannel = MakeUnique<FAgentMessageChannel>(MoveTemp(ComputeChannel));
	ChildChannel = MakeUnique<FAgentMessageChannel>(MoveTemp(SecondComputeChannel));

	bIsValid = true;
}

FUbaHordeAgent::~FUbaHordeAgent()
{
}

bool FUbaHordeAgent::BeginCommunication()
{
	ensure(bIsValid);

	// Now start the communication. From now, we can send and receive stuff.
	HordeComputeSocket->StartCommunication();

	// Let's try to read the response from the server. It should be an Attach message.
	EAgentMessageType Type = AgentChannel->ReadResponse(5000);
	UE_LOGF(LogUbaHordeAgent, Verbose, "Got a response from the server: Type=0x%08X", (int)Type);

	if (Type == EAgentMessageType::None) // Timed out
	{
		return false;
	}

	if (!ensure(Type == EAgentMessageType::Attach))
	{
		return false;
	}

	/* Fork communication channel to split communication between a main and child channels. */
	AgentChannel->Fork(100, 4*1024*1024);

	Type = ChildChannel->ReadResponse(5000);

	if (Type == EAgentMessageType::None) // Timed out
	{
		return false;
	}

	if (!ensure(Type == EAgentMessageType::Attach))
	{
		return false;
	}

	return true;
}

bool FUbaHordeAgent::UploadBinaries(const FString& BundleDirectory, const char* BundleLocator, FEvent* CancelEvent)
{
	// Prepare for giving the remote machine work.
	// 1) Send binaries to remote machine.
	// 2) Request to start executing UbaAgent.exe.
	// 3) From there, it's a simple matter of sending work over through UBA.
	ChildChannel->UploadFiles("", BundleLocator);

	TMap<FString, FArchive*> BlobFiles;

	auto FindOrAddBlobFile = [&BlobFiles, &BundleDirectory](const AgentMessage::FBlobRequest& BlobRequest) -> FArchive*
		{
			const FString Locator = FString::ConstructFromPtrSize(BlobRequest.Locator.GetData(), BlobRequest.Locator.Len());

			if (Locator.IsEmpty())
			{
				UE_LOGF(LogUbaHordeAgent, Error, "Cannot upload binaries to Horde agent with empty locator");
				return nullptr;
			}

			if (FArchive** ArchiveElem = BlobFiles.Find(Locator))
			{
				return *ArchiveElem;
			}
			else
			{
				FString Path = FPaths::Combine(BundleDirectory, Locator);
				Path.Append(TEXT(".blob"));

				if (FArchive* Archive = IFileManager::Get().CreateFileReader(*Path))
				{
					BlobFiles.Add({ Locator, Archive });
					return Archive;
				}
				else
				{
					UE_LOGF(LogUbaHordeAgent, Error, "Cannot read blob file for Horde agent upload: %ls", *Path);
					return nullptr;
				}
			}
		};

	EAgentMessageType Type = EAgentMessageType::None;

	constexpr int kReadResponseTimeoutMS = 1000;

	for (;;)
	{
		// Try to read channel response and check for cancel event periodically
		bool bTimedOut = false;
		Type = ChildChannel->ReadResponse(kReadResponseTimeoutMS, &bTimedOut);
		if (Type != EAgentMessageType::ReadBlob)
		{
			if (bTimedOut)
			{
				// If the cancel event was triggered, stop waiting for channel messages
				if (CancelEvent != nullptr && CancelEvent->Wait(0))
				{
					UE_LOGF(LogUbaHordeAgent, Log, "Uploading UBA agent binaries has been canceled");
					return false;
				}
				continue;
			}
			else
			{
				// End of stream reached
				break;
			}
		}

		AgentMessage::FBlobRequest BlobRequest;
		ChildChannel->ReadBlobRequest(BlobRequest);

		TArray<UTF8CHAR> LocatorData(BlobRequest.Locator.GetData(), BlobRequest.Locator.Len());
		LocatorData.Add(UTF8CHAR('\0'));
		UE_LOGF(LogUbaHordeAgent, Verbose, "Response [ReadBlob]: Locator=%ls, Offset=%d, Length=%d", UTF8_TO_TCHAR(LocatorData.GetData()), BlobRequest.Offset, BlobRequest.Length);

		FArchive* Archive = FindOrAddBlobFile(BlobRequest);
		if (Archive == nullptr)
		{
			return false;
		}

		Archive->Seek(BlobRequest.Offset);

		TArray<uint8> SerializedBytes;
		SerializedBytes.SetNum(int(Archive->TotalSize() - BlobRequest.Offset));
		Archive->Serialize(SerializedBytes.GetData(), SerializedBytes.Num());

		// Send the blob data.
		ChildChannel->Blob((unsigned char*)SerializedBytes.GetData(), SerializedBytes.Num());
	}

	// In case the agent upload was successful, WriteFilesResponse must have been the last received response.
	// Otherwise, the remote machine might have abruptly shutdown, which can happen quite frequently on certain cloud services.
	const bool bSuccess = (Type == EAgentMessageType::WriteFilesResponse);

	return bSuccess;
}

void FUbaHordeAgent::Execute(const char* Exe, const char* const * Args, size_t NumArgs, const char* WorkingDir, const char* const * EnvVars, size_t NumEnvVars, bool bUseWine)
{
	ChildChannel->Execute(Exe, Args, NumArgs, WorkingDir, EnvVars, NumEnvVars, bUseWine ? EExecuteProcessFlags::UseWine : EExecuteProcessFlags::None);
}

void FUbaHordeAgent::CloseConnection()
{
	// These can be null when the compute transport object was not valid in the constructor
	if (ChildChannel)
	{
		ChildChannel->Close();
	}
	if (AgentChannel)
	{
		AgentChannel->Close();
	}
}

bool FUbaHordeAgent::IsValid()
{
	return bIsValid && !bHasErrors;
}

void FUbaHordeAgent::Poll(bool LogReports, FString* OutResponseText)
{
	EAgentMessageType Type = EAgentMessageType::None;
	constexpr int32 ReadresponseTimeoutMS = 100;
	while ((Type = ChildChannel->ReadResponse(ReadresponseTimeoutMS)) != EAgentMessageType::None)
	{
		switch (Type)
		{
		case EAgentMessageType::ExecuteOutput:
		{
			if (LogReports && ChildChannel->GetResponseSize() > 0)
			{
				// Convert raw buffer to dynamic array for modification
				const ANSICHAR* ResponseDataRaw = reinterpret_cast<const ANSICHAR*>(ChildChannel->GetResponseData());
				TArray<ANSICHAR> ResponseData(ResponseDataRaw, ChildChannel->GetResponseSize());

				// Remove trailing newline characters and add NUL-terminator
				while (ResponseData.Num() > 0 && (ResponseData.Last() == '\n' || ResponseData.Last() == '\r'))
				{
					ResponseData.Pop(EAllowShrinking::No);
				}
				ResponseData.Add('\0');

				// Report output to log
				auto ResponseString = StringCast<TCHAR>(ResponseData.GetData());
				if (ResponseString.Length() > 0)
				{
					UE_LOGF(LogUbaHordeAgent, Log, "Response [ExecuteOutput]: %ls", ResponseString.Get());
				}

				if (OutResponseText)
				{
					*OutResponseText = MoveTemp(ResponseString);
					return;
				}
			}
		}
		break;

		case EAgentMessageType::ExecuteResult:
		{
			if (ChildChannel->GetResponseSize() == sizeof(int32))
			{
				if (LogReports)
				{
					const int32* ResponseData = reinterpret_cast<const int32*>(ChildChannel->GetResponseData());
					const int32 ExecuteExitCode = ResponseData[0];
					UE_LOGF(LogUbaHorde, Log, "Response [ExecuteResult]: ExitCode=%d", ExecuteExitCode);
				}
				bIsValid = false;
			}
		}
		break;

		case EAgentMessageType::Exception:
		{
			AgentMessage::FException Ex;
			ChildChannel->ReadException(Ex);
			UE_LOGF(LogUbaHorde, Log, "EXCEPTION Message: %ls", *FString(Ex.Message));
			UE_LOGF(LogUbaHorde, Log, "EXCEPTION Description: %ls", *FString(Ex.Description));
		}
		break;

		default:
		{
			// Ignore other messages here, we are only interested in the ExecuteOutput to forward reports from the remote agents to the calling process
		}
		break;
		}
	}
}
