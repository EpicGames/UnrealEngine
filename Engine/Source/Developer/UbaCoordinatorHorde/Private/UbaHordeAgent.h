// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <memory>

#include "UbaHordeMetaClient.h"

class FComputeSocket;
class FAgentMessageChannel;

DECLARE_LOG_CATEGORY_EXTERN(LogUbaHordeAgent, Log, All);

class FUbaHordeAgent
{
public:
	FUbaHordeAgent(const FHordeRemoteMachineInfo &MachineInfo);

	~FUbaHordeAgent();

	/* Start spinning the ping thread, and receive/send threads. */
	bool BeginCommunication();

	/* Upload the proper executables/binaries. The Locator is just the contents of the bundle ref file.
	 * e.g. Locator = abcdef1234567890abcdef1234567890abcdef12@34567890abcdef1234567890#123 */
	bool UploadBinaries(const FString& BundleDirectory, const char* BundleLocator, FEvent* CancelEvent = nullptr);

	void Execute(const char* Exe, const char* const * Args, size_t NumArgs, const char* WorkingDir, const char* const * EnvVars, size_t NumEnvVars, bool bUseWine = false);

	void CloseConnection(); 

	bool IsValid();

	// Reads output from the child channel and reports them to the log of the calling process.
	void Poll(bool LogReports, FString* OutResponseText = nullptr);

	inline const FHordeRemoteMachineInfo& GetMachineInfo() const
	{
		return MachineInfo;
	}

private:
	/* For Horde integration. Once all these are initialized, we're good to using UBA */
	TUniquePtr<FComputeSocket> HordeComputeSocket;
	TUniquePtr<FAgentMessageChannel> AgentChannel;
	TUniquePtr<FAgentMessageChannel> ChildChannel;

	bool bIsValid;
	bool bHasErrors;
	FHordeRemoteMachineInfo MachineInfo;
};
