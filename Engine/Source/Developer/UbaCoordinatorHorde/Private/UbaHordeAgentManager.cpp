// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaHordeAgentManager.h"
#include "HAL/Event.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformTime.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"
#include "Storage/StorageClient.h"
#include "Storage/Clients/BundleStorageClient.h"
#include "Storage/Clients/MemoryStorageClient.h"
#include "Storage/Clients/FileStorageClient.h"
#include "Storage/Nodes/ChunkNode.h"
#include "Storage/Nodes/DirectoryNode.h"
#include "Storage/BlobWriter.h"
#include "UbaBase.h"
#include "UbaHordeAgent.h"
#include "UbaDefaultConstants.h"
#include <filesystem>
#include <fstream>

namespace UbaCoordinatorHordeModule
{
	static bool bHordeForwardAgentLogs = false;
	static FAutoConsoleVariableRef CVarUbaControllerHordeForwardAgentLogs(
		TEXT("r.UbaHorde.ForwardAgentLogs"),
		bHordeForwardAgentLogs,
		TEXT("Enables or disables logging of stdout on agent side to show in controller log."));

#if PLATFORM_MAC
	constexpr uint32 EstimatedCoreCountPerInstance = 16;
#else
	constexpr uint32 EstimatedCoreCountPerInstance = 32;
#endif
}

FUbaHordeAgentManager::FUbaHordeAgentManager(const FString& InWorkingDir, const FString& InBinariesPath)
	:	WorkingDir(InWorkingDir)
	,	BinariesPath(InBinariesPath)
	,	LastRequestFailTime(0)
	,	TargetCoreCount(0)
	,	EstimatedCoreCount(0)
	,   ActiveCoreCount(0)
	,	AgentsActive(0)
	,	AgentsRequesting(0)
	,	AgentsInProgress(0)
	,	bAskForAgents(true)
{
}

FUbaHordeAgentManager::~FUbaHordeAgentManager()
{
	FScopeLock AgentsScopeLock(&AgentsLock);
	for (TUniquePtr<FHordeAgentWrapper>& Agent : Agents)
	{
		Agent->ShouldExit->Trigger();
	}

	for (TUniquePtr<FHordeAgentWrapper>& Agent : Agents)
	{
		Agent->Thread.Join();
		FGenericPlatformProcess::ReturnSynchEventToPool(Agent->ShouldExit);
	}
}

void FUbaHordeAgentManager::SetTargetCoreCount(uint32 Count)
{
	TargetCoreCount = FMath::Min((uint32)FUbaHordeConfig::Get().HordeMaxCores, Count);

	while (EstimatedCoreCount < TargetCoreCount)
	{
		if (!bAskForAgents)
		{
			return;
		}

		//UE_LOGF(LogUbaHorde, Display, "Requested new agent. Estimated core count: %u, Target core count: %u", EstimatedCoreCount.Load(), TargetCoreCount.Load());
		RequestAgent();
	}

	FScopeLock AgentsScopeLock(&AgentsLock);
	for (auto Iterator = Agents.CreateIterator(); Iterator; ++Iterator)
	{
		TUniquePtr<FHordeAgentWrapper>& Agent = *Iterator;
		if (Agent->ShouldExit->Wait(0))
		{
			Agent->Thread.Join();
			FGenericPlatformProcess::ReturnSynchEventToPool(Agent->ShouldExit);
			Iterator.RemoveCurrentSwap();
		}
	}
}

void FUbaHordeAgentManager::SetAddClientCallback(FAddClientCallback* Callback, void* UserData)
{
	AddClientCallback = Callback;
	AddClientUserData = UserData;
}

void FUbaHordeAgentManager::SetUpdateStatusCallback(FUpdateStatusCallback* Callback, void* UserData)
{
	UpdateStatusCallback = Callback;
	UpdateStatusUserData = UserData;
}

int32 FUbaHordeAgentManager::GetAgentCount() const
{
	FScopeLock AgentsScopeLock(&AgentsLock);
	return Agents.Num();
}

uint32 FUbaHordeAgentManager::GetActiveCoreCount() const
{
	return ActiveCoreCount;
}

bool FUbaHordeAgentManager::RequestCacheServer(FString& OutCacheEndpoint, FString& OutCacheSessionKey, bool& bOutWriteAccess)
{
	FScopeLock ScopeLock(&BundleRefPathsLock);
	if (!HordeMetaClient)
	{
		HordeMetaClient = MakeUnique<FUbaHordeMetaClient>();
		if (!HordeMetaClient->RefreshHttpClient())
		{
			UE_LOGF(LogUbaHorde, Warning, "Failed to create HttpClient for cache server request");
			HordeMetaClient.Reset();
			return false;
		}
	}
	ScopeLock.Unlock();

	const FUbaHordeConfig& Config = FUbaHordeConfig::Get();

	// Resolve cluster ID if set to auto
	FHordeClusterInfo ClusterInfo;
	if (Config.HordeCluster == FUbaHordeConfig::ClusterAuto)
	{
		const FString HordeRequestJsonBody = FUbaHordeMetaClient::BuildHordeRequestJsonBody(
			Config.HordeConnectionMode == EUbaHordeConnectionMode::Direct ? Config.GetHordePool() : TEXT(""),
			Config.HordeConnectionMode,
			Config.HordeEncryption,
			*Config.HordeCondition,
			false, // not exclusive for cache lookup
			Config.bHordeAllowWine
		);
		TSharedPtr<FUbaHordeMetaClient::HordeClusterPromise, ESPMode::ThreadSafe> ClusterPromise = HordeMetaClient->RequestClusterId(HordeRequestJsonBody);
		if (!ClusterPromise)
		{
			UE_LOGF(LogUbaHorde, Warning, "Failed to resolve Horde cluster ID for cache server request");
			return false;
		}
		TFuture<TTuple<FHttpResponsePtr, FHordeClusterInfo>> ClusterFuture = ClusterPromise->GetFuture();
		ClusterFuture.Wait();
		ClusterInfo = ClusterFuture.Get().Value;
	}
	else
	{
		ClusterInfo.ClusterId = Config.HordeCluster;
	}

	TSharedPtr<FUbaHordeMetaClient::HordeCacheServerPromise, ESPMode::ThreadSafe> Promise = HordeMetaClient->RequestCacheServer(*ClusterInfo.ClusterId);
	if (!Promise)
	{
		UE_LOGF(LogUbaHorde, Warning, "Failed to request Horde cache server from cluster: %ls", *ClusterInfo.ClusterId);
		return false;
	}

	TFuture<TTuple<FHttpResponsePtr, FHordeCacheServerInfo>> Future = Promise->GetFuture();
	Future.Wait();
	FHordeCacheServerInfo CacheInfo = Future.Get().Value;

	if (CacheInfo.CacheEndpoint.IsEmpty())
	{
		UE_LOGF(LogUbaHorde, Display, "No cache server available from Horde cluster: %ls", *ClusterInfo.ClusterId);
		return false;
	}

	OutCacheEndpoint = CacheInfo.CacheEndpoint;
	OutCacheSessionKey = CacheInfo.CacheSessionKey;
	bOutWriteAccess = CacheInfo.bWriteAccess;
	return true;
}

// Creates a bundle blob (one of several chunks of a file) to be uploaded to Horde
// This code has been adopted from the HordeTest project.
// See 'Engine/Source/Programs/Horde/Samples/HordeTest/Main.cpp'.
static FBlobHandleWithHash CreateHordeBundleBlob(const std::filesystem::path& Path, FBlobWriter& Writer, int64& OutLength, FIoHash& OutStreamHash)
{
	OutLength = 0;

	FChunkNodeWriter ChunkWriter(Writer);

	std::ifstream Stream(Path, std::ios::binary);

	char ReadBuffer[4096];
	while (!Stream.eof())
	{
		Stream.read(ReadBuffer, sizeof(ReadBuffer));

		const int64 ReadSize = static_cast<int64>(Stream.gcount());
		if (ReadSize == 0)
		{
			break;
		}
		OutLength += ReadSize;

		ChunkWriter.Write(FMemoryView(ReadBuffer, ReadSize));
	}

	return ChunkWriter.Flush(OutStreamHash);
}

static FDirectoryEntry CreateHordeBundleDirectoryEntry(const std::filesystem::path& Path, FBlobWriter& Writer)
{
	FDirectoryNode DirectoryNode;

	int64 BlobLength = 0;
	FIoHash StreamHash;
	FBlobHandleWithHash Target = CreateHordeBundleBlob(Path, Writer, BlobLength, StreamHash);

	EFileEntryFlags Flags = EFileEntryFlags::Executable;
	FFileEntry NewEntry(Target, FUtf8String(Path.filename().string().c_str()), Flags, BlobLength, StreamHash, FSharedBufferView());

	const FUtf8String Name = NewEntry.Name;
	const int64 Length = NewEntry.Length;
	DirectoryNode.NameToFile.Add(Name, MoveTemp(NewEntry));

	FBlobHandle DirectoryHandle = DirectoryNode.Write(Writer);

	return FDirectoryEntry(DirectoryHandle, FIoHash(), FUtf8String(Path.filename().string().c_str()), Length);
}

static bool CreateHordeBundleFromFile(const std::filesystem::path& InputFilename, const std::filesystem::path& OutputFilename)
{
	check(!InputFilename.empty());

	TSharedRef<FFileStorageClient> FileStorage = MakeShared<FFileStorageClient>(OutputFilename.parent_path());
	TSharedRef<FBundleStorageClient> Storage = MakeShared<FBundleStorageClient>(FileStorage);

	TUniquePtr<FBlobWriter> Writer = Storage->CreateWriter("");
	FDirectoryEntry RootEntry = CreateHordeBundleDirectoryEntry(InputFilename, *Writer.Get());
	Writer->Flush();

	FFileStorageClient::WriteRefToFile(OutputFilename, RootEntry.Target->GetLocator());
	return true;
}

void FUbaHordeAgentManager::RequestAgent()
{
	EstimatedCoreCount += UbaCoordinatorHordeModule::EstimatedCoreCountPerInstance;

	const int32 NextAgentIndex = Agents.Num();

	FScopeLock AgentsScopeLock(&AgentsLock);
	FHordeAgentWrapper& Wrapper = *Agents.Emplace_GetRef(MakeUnique<FHordeAgentWrapper>());

	Wrapper.ShouldExit = FGenericPlatformProcess::GetSynchEventFromPool(true);

	Wrapper.Thread = FThread(*FString::Printf(TEXT("HordeAgent %d"), NextAgentIndex), [this, &Wrapper]() { ThreadAgent(Wrapper); });
}

// Helper class to manage raw and auto-released strings for UBA agent command line arguments.
class FUbaAgentArguments
{
	TArray<const char*> RawArguments;
	TArray<FAnsiString> ArgumentStrings;
public:
	FUbaAgentArguments(int32 InitialArgumentCapapcity = 16)
	{
		RawArguments.Reserve(InitialArgumentCapapcity);
	}
	void AddRaw(const char* InArgument)
	{
		RawArguments.Add(InArgument);
	}
	void Add(const FAnsiString& InArgument)
	{
		ArgumentStrings.Add(InArgument);
	}
	void Finalize()
	{
		RawArguments.Reserve(RawArguments.Num() + ArgumentStrings.Num());
		for (const FAnsiString& Argument : ArgumentStrings)
		{
			RawArguments.Add(Argument.GetCharArray().GetData());
		}
	}
	const TArray<const char*>& Get() const
	{
		return RawArguments;
	}
};

void FUbaHordeAgentManager::ThreadAgent(FHordeAgentWrapper& Wrapper)
{
	const FUbaHordeConfig& Config = FUbaHordeConfig::Get();

	FEvent& ShouldExit = *Wrapper.ShouldExit;
	TUniquePtr<FUbaHordeAgent> Agent;
	bool bSuccess = false;

	ON_SCOPE_EXIT
	{
		if (Agent)
		{
			Agent->CloseConnection();
		}
		ShouldExit.Trigger();
	};

	int MachineCoreCount = 0;

	// If no host is specified, we need to start the agent in listen mode
	const bool bUseListen = Config.HordeHost.IsEmpty();

	FString UbaAgentIp;
	FHordeRemoteMachineInfo::FPortInfo UbaPort, UbaProxyPort;

	{
		ON_SCOPE_EXIT{ EstimatedCoreCount -= UbaCoordinatorHordeModule::EstimatedCoreCountPerInstance; };

		#if PLATFORM_WINDOWS
		const char* AppName = "UbaAgent.exe";
		#else
		const char* AppName = "UbaAgent";
		#endif

		FScopeLock ScopeLock(&BundleRefPathsLock);
		if (BundleRefPaths.IsEmpty())
		{
			struct FBundleRec
			{
				const TCHAR* Filename;
				const TCHAR* BundleRef;
				bool Optional;
			};
			const FBundleRec BundleRecs[] =
			{
#if PLATFORM_WINDOWS
				{ TEXT("UbaAgent.exe"), TEXT("UbaAgent.Bundle.ref"), false },
				{ TEXT("UbaWine.dll.so"), TEXT("UbaWine.Bundle.ref"), true },
#elif PLATFORM_LINUX
				{ TEXT("UbaAgent"), TEXT("UbaAgent.Bundle.ref"), false },
				{ TEXT("UbaAgent.debug"), TEXT("UbaAgent.debug.Bundle.ref"), true },
//				{ TEXT("libclang_rt.tsan.so"), TEXT("Tsan.Bundle.ref") }, // for debugging
#elif PLATFORM_MAC
				{ TEXT("UbaAgent"), TEXT("UbaAgent.Bundle.ref"), false },
#endif
			};

			for (const FBundleRec& Rec : BundleRecs)
			{
				const FString FilePath = FPaths::Combine(BinariesPath, Rec.Filename);
				FString BundlePath = FPaths::Combine(WorkingDir, Rec.BundleRef);

				if (!FPaths::FileExists(*FilePath))
				{
					if (Rec.Optional)
					{
						UE_LOGF(LogUbaHorde, Warning, "Failed to find optional %ls for Horde bundle", *FilePath);
						continue;
					}
					else
					{
						UE_LOGF(LogUbaHorde, Error, "Failed to create Horde bundle for: %ls", *FilePath);
						bAskForAgents = false;
						UpdateStatus(TEXT("Failed to create bundle"));
						return;
					}
				}

				if (!CreateHordeBundleFromFile(*FilePath, *BundlePath))
				{
					UE_LOGF(LogUbaHorde, Error, "Failed to create Horde bundle for: %ls", *FilePath);
					bAskForAgents = false;
					UpdateStatus(TEXT("Failed to create bundle"));
					return;
				}
				UE_LOGF(LogUbaHorde, Display, "Created Horde bundle for: %ls", *FilePath);
				BundleRefPaths.Add(BundlePath);
			}
		}

		if (!HordeMetaClient)
		{
			UpdateStatus(TEXT("Connecting..."));
			// Create Horde meta client right before we need it to make sure the CVar for the server URL has been read by now
			HordeMetaClient = MakeUnique<FUbaHordeMetaClient>();
			if (!HordeMetaClient->RefreshHttpClient())
			{
				UE_LOGF(LogUbaHorde, Error, "Failed to create HttpClient for UbaAgent");
				bAskForAgents = false;
				UpdateStatus(TEXT("Failed to connect"));
				return;
			}
			UpdateStatus(TEXT("Running"));
		}

		if (!bAskForAgents)
		{
			return;
		}

		++AgentsRequesting;
		ON_SCOPE_EXIT{ --AgentsRequesting; UpdateStatus(nullptr); };
		UpdateStatus(nullptr);

		if (LastRequestFailTime == 0)
		{
			ScopeLock.Unlock();
		}
		else
		{
			// Try to reduce pressure on horde by not asking for machines more frequent than every 5 seconds if failed to retrieve last time
			uint64 CurrentTime = FPlatformTime::Cycles64();
			uint32 MsSinceLastFail = uint32(double(CurrentTime - LastRequestFailTime) * FPlatformTime::GetSecondsPerCycle() * 1000);
			if (MsSinceLastFail < 5000)
			{
				if (ShouldExit.Wait(5000 - MsSinceLastFail))
				{
					return;
				}
			}
		}

		if (ActiveCoreCount >= TargetCoreCount)
		{
			UpdateStatus(nullptr);
			return;
		}

		// Initialize horde request with connection mode preference and encryption.
		// Pool ID is superseded by cluster ID, but it still has to be provided if default cluster and direct mode is used.
		constexpr bool bExclusiveAccess = true;
		const FString HordeRequestJsonBody = HordeMetaClient->BuildHordeRequestJsonBody(
			Config.HordeConnectionMode == EUbaHordeConnectionMode::Direct ? Config.GetHordePool() : TEXT(""),
			Config.HordeConnectionMode,
			Config.HordeEncryption,
			*Config.HordeCondition,
			bExclusiveAccess,
			Config.bHordeAllowWine
		);

		// Resolve cluster ID through Horde if "_auto" was specified
		FHordeClusterInfo ClusterInfo;
		if (Config.HordeCluster == FUbaHordeConfig::ClusterAuto)
		{
			TSharedPtr<FUbaHordeMetaClient::HordeClusterPromise, ESPMode::ThreadSafe> Promise = HordeMetaClient->RequestClusterId(HordeRequestJsonBody);
			if (!Promise)
			{
				UE_LOGF(LogUbaHorde, Log, "Failed to resolve Horde cluster ID with HTTP/Json request: %ls", *HordeRequestJsonBody);
				return;
			}
			TFuture<TTuple<FHttpResponsePtr, FHordeClusterInfo>> Future = Promise->GetFuture();
			Future.Wait();
			ClusterInfo = Future.Get().Value;
			UE_CLOGF(!ClusterInfo.ClusterId.IsEmpty(), LogUbaHorde, Verbose, "Received resolved cluster ID from Horde: %ls", *ClusterInfo.ClusterId);
		}
		else
		{
			// Otherwise, just adopt specified cluster identifier
			ClusterInfo.ClusterId = Config.HordeCluster;
		}

		// Request Horde machine for UBA agent
		TSharedPtr<FUbaHordeMetaClient::HordeMachinePromise, ESPMode::ThreadSafe> Promise = HordeMetaClient->RequestMachine(HordeRequestJsonBody, *ClusterInfo.ClusterId);
		if (!Promise)
		{
			UE_LOGF(LogUbaHorde, Log, "Failed to request Horde machine from cluster: %ls", *ClusterInfo.ClusterId);
			return;
		}
		TFuture<TTuple<FHttpResponsePtr, FHordeRemoteMachineInfo>> Future = Promise->GetFuture();
		Future.Wait();
		FHordeRemoteMachineInfo MachineInfo = Future.Get().Value;

		// If the machine couldn't be assigned, just ignore this agent slot
		if (MachineInfo.GetConnectionAddress().IsEmpty())
		{
			UE_CLOGF(!LastRequestFailTime, LogUbaHorde, Verbose, "No resources available in Horde. Will keep retrying until %u cores are used (Currently have %u)", TargetCoreCount.Load(), ActiveCoreCount.Load());
			LastRequestFailTime = FPlatformTime::Cycles64();
			UpdateStatus(nullptr);
			return;
		}

		LastRequestFailTime = 0;

		++AgentsInProgress;
		UpdateStatus(nullptr);

		ON_SCOPE_EXIT{ --AgentsInProgress; UpdateStatus(nullptr); };

		ScopeLock.Unlock();



		if (ShouldExit.Wait(0))
		{
			return;
		}

		Agent = MakeUnique<FUbaHordeAgent>(MachineInfo);

		if (!Agent->IsValid())
		{
			return;
		}

		if (!Agent->BeginCommunication())
		{
			return;
		}

		for (const FString& Bundle : BundleRefPaths)
		{
			TArray<uint8> Locator;
			if (!FFileHelper::LoadFileToArray(Locator, *Bundle))
			{
				UE_LOGF(LogUbaHorde, Error, "Cannot launch Horde processes for UBA controller because bundle path could not be found: %ls", *Bundle);
				return;
			}
			Locator.Add('\0');

			FString BundleDirectory = FPaths::GetPath(Bundle);

			if (ShouldExit.Wait(0))
			{
				return;
			}

			if (!Agent->UploadBinaries(BundleDirectory, reinterpret_cast<const char*>(Locator.GetData()), &ShouldExit))
			{
				return;
			}
		}

		// Get IP address and port for UbaAgent
		UbaAgentIp = Agent->GetMachineInfo().GetConnectionAddress();

		UbaPort = Agent->GetMachineInfo().Ports[TEXT("UbaPort")];
		UbaProxyPort = Agent->GetMachineInfo().Ports[TEXT("UbaProxyPort")];

		const uint32 UbaHostPort = uba::DefaultPort;

		// Start the UBA Agent that will connect to us, requesting for work
		FUbaAgentArguments UbaAgentArgs;

		if (bUseListen)
		{
			UbaAgentArgs.Add(FAnsiString::Printf("-Listen=%u", UbaPort.AgentPort));
			UbaAgentArgs.Add(TEXT("-ListenTimeout=10"));
		}
		else
		{
			UbaAgentArgs.Add(FAnsiString::Printf("-Host=%s:%u", TCHAR_TO_ANSI(*Config.HordeHost), UbaHostPort));
		}

		if (!Config.UbaSentryUrl.IsEmpty())
		{
			UbaAgentArgs.Add(FAnsiString::Printf("-Sentry=%s", TCHAR_TO_ANSI(*Config.UbaSentryUrl)));
		}

		UbaAgentArgs.Add(FAnsiString::Printf("-ProxyPort=%u", UbaProxyPort.AgentPort));

		// -nopoll recommended when running on remote Horde agents to make sure they exit after completion. Otherwise, it keeps running.
		UbaAgentArgs.AddRaw("-NoPoll");

		// Skip all the agent logging that would be sent over to here
		UbaAgentArgs.AddRaw("-Quiet");

		// After 15 seconds of idling agent will automatically disconnect
		UbaAgentArgs.AddRaw("-MaxIdle=15");

		#if PLATFORM_MAC
		UbaAgentArgs.Add("-KillTcpHogs");
		#endif

		UbaAgentArgs.AddRaw("-Dir=%UE_HORDE_SHARED_DIR%\\Uba");
		UbaAgentArgs.AddRaw("-Eventfile=%UE_HORDE_TERMINATION_SIGNAL_FILE%");
		UbaAgentArgs.Add(FAnsiString::Printf("-Description=%s", TCHAR_TO_ANSI(*Agent->GetMachineInfo().LeaseLink)));

		UbaAgentArgs.AddRaw("-MaxCpu=%UE_HORDE_CPU_COUNT%");
		UbaAgentArgs.AddRaw("-MulCpu=%UE_HORDE_CPU_MULTIPLIER%");
		UbaAgentArgs.AddRaw("-MaxCon=8");
		UbaAgentArgs.Add(FAnsiString::Printf("-MaxWorkers=%d", Config.HordeMaxWorkers));

		UbaAgentArgs.Finalize();

		// If the machine does not run Windows, enable the compatibility layer Wine to run UbaAgent.exe on POSIX systems
		#if PLATFORM_WINDOWS
		const bool bRunsWindowsOS = Agent->GetMachineInfo().bRunsWindowOS;
		const bool bUseWine = !bRunsWindowsOS;
		#else
		const bool bUseWine = false;
		#endif

		if (ShouldExit.Wait(0))
		{
			return;
		}

		Agent->Execute(AppName, UbaAgentArgs.Get().GetData(), (size_t)UbaAgentArgs.Get().Num(), nullptr, nullptr, 0, bUseWine);

		// Log remote execution
		FString UbaAgentCmdArgs = ANSI_TO_TCHAR(AppName);
		for (const char* Arg : UbaAgentArgs.Get())
		{
			UbaAgentCmdArgs += TEXT(" ");
			UbaAgentCmdArgs += ANSI_TO_TCHAR(Arg);
		}
		UE_LOGF(LogUbaHorde, Log, "Remote execution on Horde machine [%ls:%u]: %ls", *UbaAgentIp, UbaPort.Port, *UbaAgentCmdArgs);

		MachineCoreCount = MachineInfo.LogicalCores;
		EstimatedCoreCount += MachineCoreCount;
		ActiveCoreCount += MachineCoreCount;
		++AgentsActive;

		UpdateStatus(nullptr);
	}

	ON_SCOPE_EXIT{ EstimatedCoreCount -= MachineCoreCount; ActiveCoreCount -= MachineCoreCount; --AgentsActive; UpdateStatus(nullptr); };

	FString Response;
	bool bConnected = false;

	while (Agent->IsValid() && !ShouldExit.Wait(100))
	{
		bool bReadResponse = (bUseListen && !bConnected);
		Agent->Poll(bReadResponse || UbaCoordinatorHordeModule::bHordeForwardAgentLogs, bReadResponse ? &Response : nullptr);

		if (!bUseListen || bConnected)
		{
			continue;
		}

		if (!Response.Contains(TEXT("Listening on")))
		{
			continue;
		}

		FString CryptoNonce16;
		if (Agent->GetMachineInfo().Encryption != EUbaHordeEncryption::None)
		{
			// Generate random crypto nonce for UbaAgent
			CryptoNonce16.Reserve(16);
			for (int32 CharacterIndex = 0; CharacterIndex < 16; ++CharacterIndex)
			{
				CryptoNonce16.AppendChar(TEXT("0123456789abcdef")[FMath::Rand() % 0xF]);
			}
		}

		// Add this machine as client to the remote agent.
		// Use JSON field "port" when adding clients while "agentPort" is used for UbaAgent to listen to on the server side.
		const bool bAddClientSuccess = AddClientCallback(
			AddClientUserData,
			StringCast<uba::tchar>(*UbaAgentIp).Get(),
			static_cast<uint16>(UbaPort.Port),
			Agent->GetMachineInfo().Encryption != EUbaHordeEncryption::None ? StringCast<uba::tchar>(*CryptoNonce16).Get() : nullptr
		);

		if (!bAddClientSuccess)
		{
			UE_LOGF(LogUbaHorde, Display, "Server_AddClient(%ls:%u) failed", *UbaAgentIp, UbaPort.Port);
			return;
		}

		bConnected = true;
	}
}

void FUbaHordeAgentManager::UpdateStatus(const TCHAR* Status)
{
	if (!UpdateStatusCallback)
	{
		return;
	}

	FScopeLock Lock(&UpdateStatusLock);
	bool HasFailTime = LastRequestFailTime != 0;

	if (HasFailTime == UpdateHadFailTime && AgentsRequesting == UpdateStatusAgentsRequesting &&
		AgentsInProgress == UpdateStatusAgentsInProgress && AgentsActive == UpdateStatusAgentsActive && (!Status || UpdateStatusText == Status))
	{
		return;
	}

	UpdateHadFailTime = HasFailTime;
	UpdateStatusAgentsRequesting = AgentsRequesting;
	UpdateStatusAgentsInProgress = AgentsInProgress;
	UpdateStatusAgentsActive = AgentsActive;

	TStringBuilder<256> Temp;

	if (Status)
	{
		UpdateStatusText = Status;
	}
	else
	{
		Temp.Appendf(TEXT("%s"), *UpdateStatusText);
		if (UpdateStatusAgentsActive)
		{
			uint32 Acc = ActiveCoreCount;
			Temp.Appendf(TEXT(". %u agents (%u cores)"), UpdateStatusAgentsActive, Acc);
		}

		if (UpdateHadFailTime)
		{
			Temp.Appendf(TEXT(" - No agents available."));
		}
		else if (UpdateStatusAgentsRequesting)
		{
			Temp.Appendf(TEXT(" - Requesting %u agents..."), UpdateStatusAgentsRequesting);
		}

		if (AgentsInProgress)
		{
			Temp.Appendf(TEXT(" (Preparing %u agents)"), (uint32)AgentsInProgress);
		}
		Status = *Temp;
	}

	UpdateStatusCallback(UpdateStatusUserData, Status);
}
