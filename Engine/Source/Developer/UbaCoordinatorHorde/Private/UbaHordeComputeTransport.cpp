// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaHordeComputeTransport.h"
#include "Sockets.h"
#include "SocketSubsystem.h"

FUbaHordeComputeTransport::FUbaHordeComputeTransport(const FHordeRemoteMachineInfo& MachineInfo, bool& InHasErrors)
	: Socket(nullptr)
	, bIsClosed(false)
	, bHasErrors(InHasErrors)
{
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);

	Socket = SocketSubsystem->CreateSocket(TEXT("Stream"), TEXT(""));
	TSharedPtr<FInternetAddr> InternetAddr;

	TSharedPtr<FInternetAddr> Address = SocketSubsystem->GetAddressFromString(MachineInfo.GetConnectionAddress());
	Address->SetPort(MachineInfo.GetConnectionPort().Port);

	if (!Socket->Connect(*Address))
	{
		UE_LOGF(LogUbaHorde, Display, "Failed to connect to Horde compute service [%ls]", *Address->ToString(true));

		SocketSubsystem->DestroySocket(Socket);

		Socket = nullptr;
	}
}

FUbaHordeComputeTransport::~FUbaHordeComputeTransport()
{
	if (Socket)
	{
		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
		SocketSubsystem->DestroySocket(Socket);
	}
}

// Sends data to the remote
size_t FUbaHordeComputeTransport::Send(const void* Data, size_t Size)
{
	int32 NumBytesSent = 0;
	if (!Socket->Send((const uint8*)Data, Size, NumBytesSent))
	{
		// We can't log this since the other side could have disconnected us (causing recv to fail). This happens often in the horde setup
		//UE_LOGF(LogUbaHorde, Error, "Failed to send data to the Horde compute service: %llu %ls", (uint64)Size, Size == 1 ? TEXT("byte") : TEXT("bytes"));
		bHasErrors = true;
		return 0;
	}

	UE_LOGF(LogUbaHorde, Verbose, "Sent message to Horde compute service: %d %ls", NumBytesSent, NumBytesSent == 1 ? TEXT("byte") : TEXT("bytes"));

	return NumBytesSent;
}

// Receives data from the remote
size_t FUbaHordeComputeTransport::Recv(void* Data, size_t Size)
{
	int32 NumBytesRead = 0;
	if (!Socket->Recv((uint8*)Data, Size, NumBytesRead))
	{
		// We can't log this since the other side could have disconnected us (causing recv to fail). This happens often in the horde setup
		//if (!bIsClosed)
		//	UE_LOGF(LogUbaHorde, Error, "Failed to receive data from the Horde compute service: %llu %ls", (uint64)Size, Size == 1 ? TEXT("byte") : TEXT("bytes"));
		return 0;
	}

	UE_LOGF(LogUbaHorde, Verbose, "Received message from Horde compute service: %d %ls", NumBytesRead, NumBytesRead == 1 ? TEXT("byte") : TEXT("bytes"));

	return NumBytesRead;
}

// Indicates to the remote that no more data will be sent.
void FUbaHordeComputeTransport::MarkComplete()
{
}

// Indicates that no more data will be sent or received, and that any blocking reads/writes should stop.
void FUbaHordeComputeTransport::Close()
{
	bIsClosed = true;
	//Socket->Close();
	Socket->Shutdown(ESocketShutdownMode::ReadWrite);
}

bool FUbaHordeComputeTransport::IsValid() const
{
	return Socket != nullptr && !bHasErrors && !bIsClosed;
}
