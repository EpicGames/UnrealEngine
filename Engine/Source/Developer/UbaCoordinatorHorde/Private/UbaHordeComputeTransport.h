// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Compute/ComputeTransport.h"
#include "UbaHordeMetaClient.h"

class FSocket;

class FUbaHordeComputeTransport final : public FComputeTransport
{
public:
	// The constructor just performs a connect.
	FUbaHordeComputeTransport(const FHordeRemoteMachineInfo &Info, bool& InHasErrors);
	~FUbaHordeComputeTransport() override final;
	
	// Sends data to the remote
	size_t Send(const void* Data, size_t Size) override final;

	// Receives data from the remote
	size_t Recv(void* Data, size_t Size) override final;

	// Indicates to the remote that no more data will be sent.
	void MarkComplete() override final;

	// Indicates that no more data will be sent or received, and that any blocking reads/writes should stop.
	void Close() override final;

	bool IsValid() const override final;

private:
	FSocket* Socket;
	bool bIsClosed;
	bool& bHasErrors;
};
