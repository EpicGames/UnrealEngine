// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_SSL // OpenSSL library required for AES encryption

#include "UbaHordeComputeTransportAES.h"

#include <openssl/evp.h>
#include <openssl/aes.h>
#include <openssl/err.h>


// Message format: [length (4 bytes)][nonce (12 bytes)][encrypted data][tag (16 bytes)]
namespace CryptoFormatAES
{
	static constexpr int32 KeySize = HORDE_KEY_SIZE;
	static constexpr int32 BufferLengthSize = sizeof(int32);
	static constexpr int32 NonceSize = 12;
	static constexpr int32 NonceWords = NonceSize / sizeof(int32);
	static constexpr int32 TagSize = AES_BLOCK_SIZE; // AES-GCM tag size in bytes

	struct FPacketHeader
	{
		int32 DataLength = 0;
		uint8 Nonce[CryptoFormatAES::NonceSize] = {};
	};

	static constexpr int32 HeaderAndTagSize = sizeof(FPacketHeader) + TagSize;
} // AESFormat

#define VERIFY_OPENSSL_CALL(FUNC, ...) \
	VerifyOpenSSLStatus(FUNC(__VA_ARGS__), TEXT(#FUNC))

#define VERIFY_OPENSSL_CONTEXT_CALL(CONTEXT, FUNC, ...) \
	(CONTEXT)->VerifyOpenSSLStatus(FUNC(__VA_ARGS__), TEXT(#FUNC))

struct FUbaHordeComputeTransportAES::FOpenSSLContext
{
	EVP_CIPHER_CTX* EncryptionContext = nullptr;
	EVP_CIPHER_CTX* DecryptionContext = nullptr;

	uint8 Key[CryptoFormatAES::KeySize] = {};
	uint8 EncryptNonce[CryptoFormatAES::NonceSize] = {};
	uint8 DecryptNonce[CryptoFormatAES::NonceSize] = {};

	bool bHasErrors = false;

	FOpenSSLContext(const uint8 (&InKey)[HORDE_KEY_SIZE])
	{
		FMemory::Memcpy(Key, InKey, sizeof(InKey));

		// Initialize nonce for encryption with random numbers
		for (uint8& NonceWord : EncryptNonce)
		{
			NonceWord = (uint8)(FMath::Rand() % 256);
		}

		// Flush any potential OpenSSL error codes that were left in the queue before this context was established
		while (unsigned long ErrorCode = ERR_get_error())
		{
			UE_LOGF(LogUbaHorde, Warning, "Unhandled OpenSSL error (%lu) left in queue: %ls", ErrorCode, ANSI_TO_TCHAR(ERR_error_string(ErrorCode, nullptr)));
		}
	}

	void UpdateNonce()
	{
		// Mangle nonce to add non-determinism to message digest
		uint32* Nonce32Bits = reinterpret_cast<uint32*>(EncryptNonce);
		Nonce32Bits[0]++;
		Nonce32Bits[1]--;
		Nonce32Bits[2] = Nonce32Bits[0] ^ Nonce32Bits[1];
	}

	bool VerifyOpenSSLStatus(int ReturnCode, const TCHAR* CommandName)
	{
		if (ReturnCode != 1)
		{
			const unsigned long ErrorCode = ERR_get_error();
			UE_LOGF(LogUbaHorde, Warning, "%ls() failed with return code (%d) and error code (%lu): %ls", CommandName, ReturnCode, ErrorCode, ANSI_TO_TCHAR(ERR_error_string(ErrorCode, nullptr)));
			bHasErrors = true;
			return false;
		}
		return true;
	}

	bool EncryptBegin()
	{
		// Update nonce to disguise equal messages with non-determinism
		UpdateNonce();

		// Use nonce as initialization vector
		return VERIFY_OPENSSL_CALL(EVP_EncryptInit_ex, EncryptionContext, nullptr, nullptr, Key, EncryptNonce);
	}

	int32 Copy(void* OutData, const void* InData, int32 InDataLength)
	{
		FMemory::Memcpy(OutData, InData, (SIZE_T)InDataLength);
		return InDataLength;
	}

	int32 Encrypt(void* OutDataCipher, const void* InData, int32 InDataLength)
	{
		int OutDataLength = 0;
		if (!VERIFY_OPENSSL_CALL(EVP_EncryptUpdate, EncryptionContext, (unsigned char*)OutDataCipher, &OutDataLength, (const unsigned char*)InData, InDataLength))
		{
			return 0;
		}
		return (int32)OutDataLength;
	}

	int32 EncryptEnd(void* OutData)
	{
		int OutDataLength = 0;

		int FooterLength = 0;
		if (!VERIFY_OPENSSL_CALL(EVP_EncryptFinal_ex, EncryptionContext, (unsigned char*)OutData, &FooterLength))
		{
			return 0;
		}
		OutDataLength += FooterLength;

		if (!VERIFY_OPENSSL_CALL(EVP_CIPHER_CTX_ctrl, EncryptionContext, EVP_CTRL_GCM_GET_TAG, CryptoFormatAES::TagSize, ((char*)OutData) + OutDataLength))
		{
			return 0;
		}
		OutDataLength += CryptoFormatAES::TagSize;

		return (int32)OutDataLength;
	}

	int32 EncryptMessage(void* OutEncryptedData, const void* InData, int32 InDataLength)
	{
		int32 EncryptedDataLength = 0;

		if (EncryptBegin())
		{
			uint8* EncryptedBytes = (uint8*)OutEncryptedData;
			EncryptedDataLength += Copy(EncryptedBytes, &InDataLength, sizeof(InDataLength));
			EncryptedDataLength += Copy(EncryptedBytes + EncryptedDataLength, EncryptNonce, sizeof(EncryptNonce));
			EncryptedDataLength += Encrypt(EncryptedBytes + EncryptedDataLength, InData, InDataLength);
			EncryptedDataLength += EncryptEnd(EncryptedBytes + EncryptedDataLength);
		}

		return EncryptedDataLength;
	}

	bool DecryptBegin(const void* InNonce, int32 InNonceSize)
	{
		// Use nonce that was received with the encrypted message
		check(InNonce != nullptr);
		check(InNonceSize == CryptoFormatAES::NonceSize);
		FMemory::Memcpy(DecryptNonce, InNonce, (SIZE_T)InNonceSize);

		// Use nonce as initialization vector
		return VERIFY_OPENSSL_CALL(EVP_DecryptInit_ex, DecryptionContext, nullptr, nullptr, Key, DecryptNonce);
	}

	int32 Decrypt(void* OutDataPlaintext, const void* InData, int32 InDataLength)
	{
		int OutDataLength = 0;
		if (!VERIFY_OPENSSL_CALL(EVP_DecryptUpdate, DecryptionContext, (unsigned char*)OutDataPlaintext, &OutDataLength, (const unsigned char*)InData, InDataLength))
		{
			return 0;
		}
		return (int32)OutDataLength;
	}

	int32 DecryptEnd(void* OutDataPlaintext)
	{
		int OutDataLength = 0;
		unsigned char* OutBytes = (unsigned char*)OutDataPlaintext;

		unsigned char Tag[CryptoFormatAES::TagSize];
		FMemory::Memcpy(Tag, OutBytes, sizeof(Tag));
		if (!VERIFY_OPENSSL_CALL(EVP_CIPHER_CTX_ctrl, DecryptionContext, EVP_CTRL_GCM_SET_TAG, CryptoFormatAES::TagSize, Tag))
		{
			return 0;
		}
		OutDataLength += CryptoFormatAES::TagSize;

		int FooterLength = 0;
		if (!VERIFY_OPENSSL_CALL(EVP_DecryptFinal_ex, DecryptionContext, OutBytes + OutDataLength, &FooterLength))
		{
			return 0;
		}
		OutDataLength += FooterLength;

		return (int32)OutDataLength;
	}
};

static EVP_CIPHER_CTX* NewAESCryptoContext()
{
	EVP_CIPHER_CTX* CryptoContext = EVP_CIPHER_CTX_new();
	EVP_CIPHER_CTX_init(CryptoContext);
	return CryptoContext;
}

FUbaHordeComputeTransportAES::FUbaHordeComputeTransportAES(const FHordeRemoteMachineInfo& MachineInfo, TUniquePtr<FComputeTransport>&& InInnerTransport)
	: OpenSSLContext(new FOpenSSLContext(MachineInfo.Key))
	, InnerTransport(MoveTemp(InInnerTransport))
	, RemainingDataOffset(0)
	, bIsClosed(false)
{
	// Initialize AES encryption and decrption contexts
	int Result = 0;
	OpenSSLContext->EncryptionContext = NewAESCryptoContext();
	VERIFY_OPENSSL_CONTEXT_CALL(OpenSSLContext, EVP_EncryptInit_ex, OpenSSLContext->EncryptionContext, EVP_aes_256_gcm(), nullptr, nullptr, nullptr);

	OpenSSLContext->DecryptionContext = NewAESCryptoContext();
	VERIFY_OPENSSL_CONTEXT_CALL(OpenSSLContext, EVP_DecryptInit_ex, OpenSSLContext->DecryptionContext, EVP_aes_256_gcm(), nullptr, nullptr, nullptr);
}

FUbaHordeComputeTransportAES::~FUbaHordeComputeTransportAES()
{
	if (OpenSSLContext)
	{
		if (OpenSSLContext->EncryptionContext)
		{
			EVP_CIPHER_CTX_free(OpenSSLContext->EncryptionContext);
		}
		if (OpenSSLContext->DecryptionContext)
		{
			EVP_CIPHER_CTX_free(OpenSSLContext->DecryptionContext);
		}
		delete OpenSSLContext;
	}
}

// Sends data to the remote
size_t FUbaHordeComputeTransportAES::Send(const void* Data, size_t Size)
{
	if (!IsValid())
	{
		return 0;
	}

	FScopeLock LockGuard(&IntermediateBuffersLock);

	// Encrypt data before sending
	int32 DataLength = (int32)Size;
	int32 MessageLength = CryptoFormatAES::HeaderAndTagSize + DataLength;
	uint8* EncryptedData = GetAndResizeEncryptedBuffer(MessageLength);

	int32 EncryptedDataLength = OpenSSLContext->EncryptMessage(EncryptedData, Data, DataLength);
	if (EncryptedDataLength == 0)
	{
		return 0;
	}

	// Send encrypted data over common transport layer
	if (!InnerTransport->SendMessage(EncryptedData, (size_t)EncryptedDataLength))
	{
		return 0;
	}

	return DataLength;
}

// Receives data from the remote
size_t FUbaHordeComputeTransportAES::Recv(void* Data, size_t Size)
{
	if (!IsValid())
	{
		return 0;
	}

	// Check if there is remaining decrypted data from previous call
	if (!RemainingData.IsEmpty())
	{
		FScopeLock LockGuard(&IntermediateBuffersLock);

		const int32 NumBytesFromRemainingData = FMath::Min(RemainingData.Num() - RemainingDataOffset, (int32)Size);

		// Return remaining data from intermediate buffer
		FMemory::Memcpy(Data, &RemainingData[RemainingDataOffset], (size_t)NumBytesFromRemainingData);

		// When all remaining data has been consumed, reset intermediate buffer
		RemainingDataOffset += NumBytesFromRemainingData;

		if (RemainingDataOffset >= RemainingData.Num())
		{
			RemainingData.Empty();
		}

		return (size_t)NumBytesFromRemainingData;
	}

	// Receive message header
	CryptoFormatAES::FPacketHeader Header;

	if (!InnerTransport->RecvMessage(&Header, sizeof(Header)))
	{
		UE_LOGF(LogUbaHorde, VeryVerbose, "Failed to receive AES packet (Expected size = %d)", (int32)Size);
		return 0;
	}

	FScopeLock LockGuard(&IntermediateBuffersLock);

	// Recieve message data
	int32 MessageLength = CryptoFormatAES::TagSize + Header.DataLength;
	uint8* EncryptedData = GetAndResizeEncryptedBuffer(MessageLength);

	if (!InnerTransport->RecvMessage(EncryptedData, (size_t)MessageLength))
	{
		UE_LOGF(LogUbaHorde, VeryVerbose, "Failed to receive AES packet (Expected size = %d, Size specified in header = %d)", (int32)Size, Header.DataLength);
		return 0;
	}

	if (!OpenSSLContext->DecryptBegin(Header.Nonce, sizeof(Header.Nonce)))
	{
		return 0;
	}

	// Decrypt data and store remaining encrypted data for next Recv() call
	const int32 NumBytesProcessed = FMath::Min(Header.DataLength, (int32)Size);

	OpenSSLContext->Decrypt(Data, EncryptedData, NumBytesProcessed);

	if (Header.DataLength > NumBytesProcessed)
	{
		RemainingDataOffset = 0;
		RemainingData.SetNumUninitialized(Header.DataLength - NumBytesProcessed);
		OpenSSLContext->Decrypt(RemainingData.GetData(), EncryptedData + NumBytesProcessed, RemainingData.Num());
	}

	OpenSSLContext->DecryptEnd(EncryptedData + Header.DataLength);

	return NumBytesProcessed;
}

// Indicates to the remote that no more data will be sent.
void FUbaHordeComputeTransportAES::MarkComplete()
{
	if (IsValid())
	{
		InnerTransport->MarkComplete();
	}
}

// Indicates that no more data will be sent or received, and that any blocking reads/writes should stop.
void FUbaHordeComputeTransportAES::Close()
{
	if (!bIsClosed)
	{
		if (IsValid())
		{
			InnerTransport->Close();
		}
		bIsClosed = true;
	}
}

bool FUbaHordeComputeTransportAES::IsValid() const
{
	return InnerTransport.IsValid() && InnerTransport->IsValid() && OpenSSLContext != nullptr && !OpenSSLContext->bHasErrors;
}

uint8* FUbaHordeComputeTransportAES::GetAndResizeEncryptedBuffer(int32 Size)
{
	if (Size > EncryptedBuffer.Num())
	{
		EncryptedBuffer.SetNumUninitialized(Size);
	}
	return EncryptedBuffer.GetData();
}

#undef VERIFY_OPENSSL_CALL
#undef VERIFY_OPENSSL_CONTEXT_CALL

#endif // WITH_SSL

