// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_SSL

#include "Compute/ComputeTransport.h"
#include "UbaHordeMetaClient.h"
#include "UbaHordeComputeTransport.h"

class FUbaHordeComputeTransportAES final : public FComputeTransport
{
public:
	// The constructor just performs a connect.
	FUbaHordeComputeTransportAES(const FHordeRemoteMachineInfo &Info, TUniquePtr<FComputeTransport>&& InInnerTransport);
	~FUbaHordeComputeTransportAES() override final;
	
	// Sends data to the remote
	size_t Send(const void* Data, size_t Size) override final;

	// Receives data from the remote
	size_t Recv(void* Data, size_t Size) override final;

	// Indicates to the remote that no more data will be sent.
	void MarkComplete() override final;

	// Indicates that no more data will be sent or received, and that any blocking reads/writes should stop.
	void Close() override final;

	bool IsValid() const override final;

private:
	struct FOpenSSLContext;

	uint8* GetAndResizeEncryptedBuffer(int32 Size);

	FOpenSSLContext* OpenSSLContext;
	TUniquePtr<FComputeTransport> InnerTransport;
	TArray<uint8> EncryptedBuffer;
	TArray<uint8> RemainingData;
	int32 RemainingDataOffset;
	FCriticalSection IntermediateBuffersLock;
	bool bIsClosed;
};

#endif // WITH_SSL
