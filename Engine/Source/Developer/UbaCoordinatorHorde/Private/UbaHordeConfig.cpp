// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaHordeConfig.h"
#include "UbaHordeMetaClient.h"

#include "HAL/PlatformProcess.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"
#include "XmlParser.h"


const TCHAR* FUbaHordeConfig::ClusterDefault = TEXT("default");
const TCHAR* FUbaHordeConfig::ClusterAuto = TEXT("_auto");

UBACOORDINATORHORDE_API const FString& FUbaHordeConfig::GetHordePool() const
{
	return HordePool;
}

struct FUbaHordeConfigParser
{
	template <typename T>
	struct TParameter
	{
		T Value{};
		const TCHAR* Source{};
	};

	const TCHAR* CurrentSource{};

	TParameter<FString> Provider;
	TParameter<FString> ProviderEnabled;
	TParameter<FString> Pool;
	TParameter<FString> Server;
	TParameter<FString> Token;
	TParameter<FString> Condition;
	TParameter<FString> Cluster;
	TParameter<FString> Host;
	TParameter<FString> SentryUrl;
	TParameter<int32> MaxCores;
	TParameter<int32> MaxWorkers;
	TParameter<int32> MaxParallelActions;
	TParameter<bool> bAllowWine;
	TParameter<FString> ConnectionMode;
	TParameter<FString> Encryption;

	template <typename T>
	void SetConfig(TParameter<T>& OutParameter, T InValue, const TCHAR* InSource = nullptr)
	{
		OutParameter.Value = MoveTemp(InValue);
		OutParameter.Source = InSource != nullptr ? InSource : CurrentSource;
	}

	template <typename T>
	void GetConfig(TParameter<T>& Parameter, T& OutValue)
	{
		if (Parameter.Source)
		{
			OutValue = MoveTemp(Parameter.Value);
		}
	}

	bool GetUbaConfigString(const TCHAR* Section, const TCHAR* Key, TParameter<FString>& OutParameter)
	{
		if (GConfig->GetString(Section, Key, OutParameter.Value, GEngineIni) || GConfig->GetString(Section, *FString::Printf(TEXT("Uba%s"), Key), OutParameter.Value, GEngineIni))
		{
			OutParameter.Source = CurrentSource;
			return true;
		}
		return false;
	}

	bool GetUbaConfigInt(const TCHAR* Section, const TCHAR* Key, TParameter<int32>& OutParameter)
	{
		if (GConfig->GetInt(Section, Key, OutParameter.Value, GEngineIni) || GConfig->GetInt(Section, *FString::Printf(TEXT("Uba%s"), Key), OutParameter.Value, GEngineIni))
		{
			OutParameter.Source = CurrentSource;
			return true;
		}
		return false;
	}

	bool GetUbaConfigBool(const TCHAR* Section, const TCHAR* Key, TParameter<bool>& OutParameter)
	{
		if (GConfig->GetBool(Section, Key, OutParameter.Value, GEngineIni) || GConfig->GetBool(Section, *FString::Printf(TEXT("Uba%s"), Key), OutParameter.Value, GEngineIni))
		{
			OutParameter.Source = CurrentSource;
			return true;
		}
		return false;
	}

	bool ParseCmdLineString(const TCHAR* CmdLine, const TCHAR* Parameter, TParameter<FString>& OutParameter)
	{
		if (FParse::Value(CmdLine, Parameter, OutParameter.Value))
		{
			OutParameter.Source = CurrentSource;
			return true;
		}
		return false;
	}

	bool ParseCmdLineInt(const TCHAR* CmdLine, const TCHAR* Parameter, TParameter<int32>& OutParameter)
	{
		if (FParse::Value(CmdLine, Parameter, OutParameter.Value))
		{
			OutParameter.Source = CurrentSource;
			return true;
		}
		return false;
	}

	bool ParseCmdLineBoolean(const TCHAR* CmdLine, const TCHAR* Parameter, TParameter<bool>& OutParameter)
	{
		if (FParse::Param(CmdLine, Parameter))
		{
			FString ArgumentValue;
			if (FParse::Value(CmdLine, *FString::Printf(TEXT("%s="), Parameter), ArgumentValue))
			{
				OutParameter.Value = ArgumentValue != TEXT("false") && ArgumentValue != TEXT("0");
			}
			else
			{
				OutParameter.Value = true;
			}
			OutParameter.Source = CurrentSource;
			return true;
		}
		return false;
	}

	bool ParseConfigFromXmlFile(const TCHAR* InBuildConfigFilename);
	void ParseConfigFromIniFile();
	void ParseConfigFromCommandline(const TCHAR* CmdLine);
};

bool FUbaHordeConfigParser::ParseConfigFromXmlFile(const TCHAR* InBuildConfigFilename)
{
	CurrentSource = TEXT("XML");

	FXmlFile XmlFile;
	if (!XmlFile.LoadFile(InBuildConfigFilename))
	{
		return false;
	}

	static const FString ConfigurationTag(TEXT("Configuration"));
	static const FString BuildConfigurationTag(TEXT("BuildConfiguration"));
	static const FString MaxParallelActionsTag(TEXT("MaxParallelActions"));
	static const FString HordeTag(TEXT("Horde"));
	static const FString UbaDisabledTag(TEXT("bDisableHorde"));
	static const FString ServerTag(TEXT("Server"));
	static const FString ConnectionModeTag(TEXT("ConnectionMode"));
	static const FString EncryptionTag(TEXT("Encryption"));
	static const FString ClusterTag(TEXT("Cluster"));
	static const FString PoolTag(TEXT("Pool"));
	static const FString UBASentryUrlTag(TEXT("UBASentryUrl"));

	const FXmlNode* ConfigurationNode = XmlFile.GetRootNode();
	if (ConfigurationNode == nullptr || ConfigurationNode->GetTag() != ConfigurationTag)
	{
		return false;
	}

	if (const FXmlNode* BuildConfigurationNode = ConfigurationNode->FindChildNode(BuildConfigurationTag))
	{
		if (const FXmlNode* MaxParallelActionsNode = BuildConfigurationNode->FindChildNode(MaxParallelActionsTag))
		{
			SetConfig(MaxParallelActions, FCString::Atoi(*MaxParallelActionsNode->GetContent()));
		}
	}

	if (const FXmlNode* HordeNode = ConfigurationNode->FindChildNode(HordeTag))
	{
		if (const FXmlNode* UbaDisabledNode = HordeNode->FindChildNode(UbaDisabledTag))
		{
			if (UbaDisabledNode->GetContent() == TEXT("true"))
			{
				SetConfig(ProviderEnabled, FString(TEXT("Disabled")));
			}
		}
		if (const FXmlNode* PoolNode = HordeNode->FindChildNode(PoolTag))
		{
			SetConfig(Pool, PoolNode->GetContent());
		}
		if (const FXmlNode* ServerNode = HordeNode->FindChildNode(ServerTag))
		{
			SetConfig(Server, ServerNode->GetContent());
		}
		if (const FXmlNode* ConnectionModeNode = HordeNode->FindChildNode(ConnectionModeTag))
		{
			SetConfig(ConnectionMode, ConnectionModeNode->GetContent());
		}
		if (const FXmlNode* EncryptionNode = HordeNode->FindChildNode(EncryptionTag))
		{
			SetConfig(Encryption, EncryptionNode->GetContent());
		}
		if (const FXmlNode* ClusterNode = HordeNode->FindChildNode(ClusterTag))
		{
			SetConfig(Cluster, ClusterNode->GetContent());
		}
		if (const FXmlNode* UBASentryUrlNode = HordeNode->FindChildNode(UBASentryUrlTag))
		{
			SetConfig(SentryUrl, UBASentryUrlNode->GetContent());
		}
	}

	return true;
}

void FUbaHordeConfigParser::ParseConfigFromIniFile()
{
	CurrentSource = TEXT("INI");

	// Read array of Horde providers and use default is none is set. Only select the first available since UbaController does not support multiple providers.
	TArray<FString> ProviderList;
	GConfig->GetArray(TEXT("UbaController"), GIsBuildMachine ? TEXT("BuildMachineProviders") : TEXT("Providers"), ProviderList, GEngineIni);

	// Command line can override the UBA provider from INI settings
	FString OverrideProviders;
	if (FParse::Value(FCommandLine::Get(), TEXT("UBAProviders="), OverrideProviders) && OverrideProviders.ParseIntoArray(ProviderList, TEXT("+")) > 0)
	{
		SetConfig(Provider, ProviderList[0], TEXT("CMD"));
	}
	else if (FParse::Param(FCommandLine::Get(), TEXT("UBARelay")))
	{
		// Shortcut arguennt to enable default provider for relay mode
		SetConfig(Provider, FString(TEXT("Uba.Provider.Horde.Relay")), TEXT("CMD"));
	}
	else if (!ProviderList.IsEmpty())
	{
		SetConfig(Provider, ProviderList[0]);
	}
	else
	{
		// Fallback to default provider
		SetConfig(Provider, FString(TEXT("Uba.Provider.Horde")));
	}

	GetUbaConfigString(*Provider.Value, TEXT("Enabled"), ProviderEnabled);

	// Also check [UbaController]:Enabled for compatibility
	if (GetUbaConfigString(TEXT("UbaController"), TEXT("Enabled"), ProviderEnabled))
	{
		UE_LOGF(LogUbaHorde, Warning, "Configuration '[UbaController]:Enabled' is deprecated since UE 5.6; Add an enabled Horde provider to '[UbaController]:+Providers' instead");
	}

	// Read all configurations from provider section
	GetUbaConfigString(*Provider.Value, TEXT("ServerUrl"), Server);
	GetUbaConfigString(*Provider.Value, TEXT("Token"), Token);
	GetUbaConfigString(*Provider.Value, TEXT("Pool"), Pool);
	GetUbaConfigString(*Provider.Value, TEXT("Cluster"), Cluster);
	GetUbaConfigString(*Provider.Value, TEXT("LocalHost"), Host);
	GetUbaConfigInt(*Provider.Value, TEXT("MaxCores"), MaxCores);
	GetUbaConfigInt(*Provider.Value, TEXT("MaxWorkers"), MaxWorkers);
	GetUbaConfigBool(*Provider.Value, TEXT("AllowWine"), bAllowWine);
	GetUbaConfigString(*Provider.Value, TEXT("ConnectionMode"), ConnectionMode);
	GetUbaConfigString(*Provider.Value, TEXT("Encryption"), Encryption);
	GetUbaConfigString(*Provider.Value, TEXT("SentryUrl"), SentryUrl);
}

void FUbaHordeConfigParser::ParseConfigFromCommandline(const TCHAR* CmdLine)
{
	CurrentSource = TEXT("CMD");

	ParseCmdLineString(CmdLine, TEXT("UBAHorde="), Server);
	ParseCmdLineString(CmdLine, TEXT("UBAHordeToken="), Token);
	ParseCmdLineString(CmdLine, TEXT("UBAHordePool="), Pool);
	ParseCmdLineBoolean(CmdLine, TEXT("UBAHordeAllowWine"), bAllowWine);
	ParseCmdLineInt(CmdLine, TEXT("UBAHordeMaxCores="), MaxCores);
	ParseCmdLineInt(CmdLine, TEXT("UBAHordeMaxWorkers="), MaxWorkers);
	ParseCmdLineString(CmdLine, TEXT("UBAHordeHost="), Host);
	ParseCmdLineString(CmdLine, TEXT("UBAHordeCluster="), Cluster);
	ParseCmdLineString(CmdLine, TEXT("UBAHordeRequirements="), Condition);
	ParseCmdLineString(CmdLine, TEXT("UBAHordeConnectionMode="), ConnectionMode);
	ParseCmdLineString(CmdLine, TEXT("UBAHordeEncryption="), Encryption);
	ParseCmdLineString(CmdLine, TEXT("UBASentryUrl="), SentryUrl);
	if (FParse::Param(CmdLine, TEXT("UBADisableHorde")))
	{
		ProviderEnabled.Value = TEXT("False");
		ProviderEnabled.Source = CurrentSource;
	}
	else if (FParse::Param(CmdLine, TEXT("UBAEnableHorde")))
	{
		ProviderEnabled.Value = TEXT("True");
		ProviderEnabled.Source = CurrentSource;
	}

	ParseCmdLineInt(CmdLine, TEXT("MaxParallelActions="), MaxParallelActions);
	if (FParse::Param(CmdLine, TEXT("ExclusiveRemoteShaderCompiling")))
	{
		MaxParallelActions.Value = 0; // -ExclusiveRemoteShaderCompiling overrides -MaxParallelActions
		MaxParallelActions.Source = CurrentSource;
	}
}

// Matches same list of XML files to parse the configuration from as in XmlConfig.cs
static TArray<FString> GetBuildConfigFilePaths()
{
	//	"C:\ProgramData\Unreal Engine\UnrealBuildTool\BuildConfiguration.xml"
	const TCHAR* UEFolderName = TEXT("Unreal Engine");
	const TCHAR* UBTFolderName = TEXT("UnrealBuildTool");
	const TCHAR* BuildConfigFilename = TEXT("BuildConfiguration.xml");

	TArray<FString> OutPaths;

	// Skip all the config files under the Engine folder if it's an installed build
	if (!FApp::IsEngineInstalled())
	{
		// Check for the engine config file under /Engine/Programs/NotForLicensees/UnrealBuildTool
		OutPaths.Add(FPaths::Combine(FPaths::EngineDir(), TEXT("Restricted"), TEXT("NotForLicensees"), TEXT("Programs"), UBTFolderName, BuildConfigFilename));

		// Check for the engine user config file under /Engine/Saved/UnrealBuildTool
		OutPaths.Add(FPaths::Combine(FPaths::EngineDir(), TEXT("Saved"), UBTFolderName, BuildConfigFilename));
	}

	// Check for the global config file under ProgramData/Unreal Engine/UnrealBuildTool
	const FString ProgramDataFolder = FPlatformProcess::GetApplicationSettingsDir(FPlatformProcess::ApplicationSettingsContext{ FPlatformProcess::ApplicationSettingsContext::Context::ApplicationSpecific, /*bIsEpic:*/ false });
	if (!ProgramDataFolder.IsEmpty())
	{
		OutPaths.Add(FPaths::Combine(ProgramDataFolder, UEFolderName, UBTFolderName, BuildConfigFilename));
	}

	// Check for the global config file under AppData/Unreal Engine/UnrealBuildTool (Roaming)
	const FString AppDataFolder = FPlatformProcess::GetApplicationSettingsDir(FPlatformProcess::ApplicationSettingsContext{ FPlatformProcess::ApplicationSettingsContext::Context::RoamingUser, /*bIsEpic:*/ false });
	if (!AppDataFolder.IsEmpty())
	{
		OutPaths.Add(FPaths::Combine(AppDataFolder, UEFolderName, UBTFolderName, BuildConfigFilename));
	}

	// Check for the global config file under LocalAppData/Unreal Engine/UnrealBuildTool
	if (const TCHAR* LocalAppDataFolder = FPlatformProcess::UserSettingsDir())
	{
		OutPaths.Add(FPaths::Combine(LocalAppDataFolder, UEFolderName, UBTFolderName, BuildConfigFilename));
	}

	// Check for the global config file under My Documents/Unreal Engine/UnrealBuildTool
	if (const TCHAR* PersonalFolder = FPlatformProcess::UserDir())
	{
		OutPaths.Add(FPaths::Combine(PersonalFolder, UEFolderName, UBTFolderName, BuildConfigFilename));
	}

	return OutPaths;
}

static bool IsUbaProviderEnabled(const TCHAR* InEnabledState)
{
	EUbaHordeEnabledMode HordeEnabledMode = EUbaHordeEnabledMode::Disabled;
	LexFromString(HordeEnabledMode, InEnabledState);
	return HordeEnabledMode == EUbaHordeEnabledMode::Enabled || (HordeEnabledMode == EUbaHordeEnabledMode::EnabledForBuildMachineOnly && GIsBuildMachine);
}

/**
 * Reads the UBA / Horde configuration in the following order :
 *  1. From a list of BuildConfiguration.xml files (in application settings and user folders)
 *  2. From BaseEngine.ini configuration files under [Horde] section
 *  3. From command line argument (e.g. -UBAHorde=...)
 */
void FUbaHordeConfig::Initialize()
{
	FUbaHordeConfigParser ConfigParser;

	// Read configurations from BuildConfiguration.xml files
	for (const FString& ConfigFilePath : GetBuildConfigFilePaths())
	{
		if (ConfigParser.ParseConfigFromXmlFile(*ConfigFilePath))
		{
			UE_LOGF(LogUbaHorde, Verbose, "Read UBA/Horde settings from %ls", *ConfigFilePath);
		}
	}

	// Override configuration entries by their configuration INI files
	ConfigParser.ParseConfigFromIniFile();

	// Override configuration entries by their command line arguments
	ConfigParser.ParseConfigFromCommandline(FCommandLine::Get());

	// Convert parsed argumetns to public configuration parameters
	ConfigParser.GetConfig(ConfigParser.Provider, Provider);
	bIsProviderEnabled = IsUbaProviderEnabled(*ConfigParser.ProviderEnabled.Value);
	ConfigParser.GetConfig(ConfigParser.Pool, HordePool);
	ConfigParser.GetConfig(ConfigParser.Server, HordeServer);
	ConfigParser.GetConfig(ConfigParser.Token, HordeToken);
	ConfigParser.GetConfig(ConfigParser.Condition, HordeCondition);
	ConfigParser.GetConfig(ConfigParser.Cluster, HordeCluster);
	ConfigParser.GetConfig(ConfigParser.Host, HordeHost);
	ConfigParser.GetConfig(ConfigParser.SentryUrl, UbaSentryUrl);
	ConfigParser.GetConfig(ConfigParser.MaxCores, HordeMaxCores);
	ConfigParser.GetConfig(ConfigParser.MaxWorkers, HordeMaxWorkers);
	ConfigParser.GetConfig(ConfigParser.MaxParallelActions, MaxParallelActions);
	ConfigParser.GetConfig(ConfigParser.bAllowWine, bHordeAllowWine);

	if (ConfigParser.ConnectionMode.Source)
	{
		LexFromString(HordeConnectionMode, *ConfigParser.ConnectionMode.Value);
	}
	if (ConfigParser.Encryption.Source)
	{
		LexFromString(HordeEncryption, *ConfigParser.Encryption.Value);
	}

	constexpr const TCHAR* kConfigSourceImplied = TEXT("Implied");

	// If connection mode is "realy"-mode, encryption is implied and we use AES since UbaController does not support SSL/TLS yet
	if (HordeConnectionMode == EUbaHordeConnectionMode::Relay && HordeEncryption == EUbaHordeEncryption::None)
	{
		HordeEncryption = EUbaHordeEncryption::AES;
		ConfigParser.Encryption.Source = kConfigSourceImplied;
	}

	if (bIsProviderEnabled)
	{
		auto GetConfigValue = [](const TCHAR* InValue) -> const TCHAR*
			{
				return InValue && *InValue != TEXT('\0') ? InValue : TEXT("<None>");
			};

		auto GetConfigSource = [](const TCHAR* InSource) -> const TCHAR*
			{
				return InSource && *InSource != TEXT('\0') ? InSource : TEXT("Default");
			};

		UE_LOGF(
			LogUbaHorde, Display,
			"UBA/Horde Configuration [%ls]:" "\n  Server:            %ls (%ls)" "\n  Pool:              %ls (%ls)" "\n  Condition:         %ls (%ls)" "\n  Cluster:           %ls (%ls)" "\n  Host:              %ls (%ls)" "\n  MaxCores:          %d (%ls)" "\n  MaxWorkers:        %d (%ls)" "\n  Allow Wine:        %ls (%ls)" "\n  Mode:              %ls (%ls)" "\n  Encryption:        %ls (%ls)"
			, *Provider
			, GetConfigValue(*HordeServer), GetConfigSource(ConfigParser.Server.Source)
			, GetConfigValue(*GetHordePool()), GetConfigSource(ConfigParser.Pool.Source)
			, GetConfigValue(*HordeCondition), GetConfigSource(ConfigParser.Condition.Source)
			, GetConfigValue(*HordeCluster), GetConfigSource(ConfigParser.Cluster.Source)
			, GetConfigValue(*HordeHost), GetConfigSource(ConfigParser.Host.Source)
			, HordeMaxCores, GetConfigSource(ConfigParser.MaxCores.Source)
			, HordeMaxWorkers, GetConfigSource(ConfigParser.MaxWorkers.Source)
			, bHordeAllowWine ? TEXT("Yes") : TEXT("No"), GetConfigSource(ConfigParser.bAllowWine.Source)
			, GetConfigValue(LexToString(HordeConnectionMode)), GetConfigSource(ConfigParser.ConnectionMode.Source)
			, GetConfigValue(LexToString(HordeEncryption)), GetConfigSource(ConfigParser.Encryption.Source)
		);
	}
	else
	{
		UE_LOGF(LogUbaHorde, Display, "UBA/Horde Configuration [%ls]: Not Enabled", *Provider);
	}
}

UBACOORDINATORHORDE_API bool FUbaHordeConfig::IsHordeEnabled() const
{
	return bIsProviderEnabled;
}

UBACOORDINATORHORDE_API const FUbaHordeConfig& FUbaHordeConfig::Get()
{
	static FUbaHordeConfig Config;
	static FRWLock InitializedLock;
	static bool bInitialized = false;

	{
		FRWScopeLock LockGuard(InitializedLock, FRWScopeLockType::SLT_ReadOnly);
		if (bInitialized)
		{
			return Config;
		}
	}

	{
		FRWScopeLock LockGuard(InitializedLock, FRWScopeLockType::SLT_Write);
		if (!bInitialized)
		{
			Config.Initialize();
			bInitialized = true;
		}
	}

	return Config;
}

UBACOORDINATORHORDE_API const TCHAR* LexToString(EUbaHordeConnectionMode InConnectionMode)
{
	switch (InConnectionMode)
	{
	case EUbaHordeConnectionMode::Direct:	return TEXT("direct");
	case EUbaHordeConnectionMode::Tunnel:	return TEXT("tunnel");
	case EUbaHordeConnectionMode::Relay:	return TEXT("relay");
	}
	return TEXT("");
}

UBACOORDINATORHORDE_API bool LexFromString(EUbaHordeConnectionMode& OutConnectionMode, const TCHAR* InString)
{
	if (FCString::Stricmp(InString, TEXT("direct")) == 0)
	{
		OutConnectionMode = EUbaHordeConnectionMode::Direct;
	}
	else if (FCString::Stricmp(InString, TEXT("tunnel")) == 0)
	{
		OutConnectionMode = EUbaHordeConnectionMode::Tunnel;
	}
	else if (FCString::Stricmp(InString, TEXT("relay")) == 0)
	{
		OutConnectionMode = EUbaHordeConnectionMode::Relay;
	}
	else
	{
		return false;
	}
	return true;
}

UBACOORDINATORHORDE_API const TCHAR* LexToString(EUbaHordeEncryption InTransportEncrypted)
{
	switch (InTransportEncrypted)
	{
	case EUbaHordeEncryption::None:	break;
	case EUbaHordeEncryption::AES: return TEXT("aes");
	}
	return TEXT("");
}

UBACOORDINATORHORDE_API bool LexFromString(EUbaHordeEncryption& OutEncryption, const TCHAR* InString)
{
	if (FCString::Stricmp(InString, TEXT("NONE")) == 0)
	{
		OutEncryption = EUbaHordeEncryption::None;
	}
	else if (FCString::Stricmp(InString, TEXT("AES")) == 0)
	{
		OutEncryption = EUbaHordeEncryption::AES;
	}
	else
	{
		return false;
	}
	return true;
}

UBACOORDINATORHORDE_API const TCHAR* LexToString(EUbaHordeEnabledMode InEnabledMode)
{
	switch (InEnabledMode)
	{
	case EUbaHordeEnabledMode::Enabled: return TEXT("True");
	case EUbaHordeEnabledMode::Disabled: return TEXT("False");
	case EUbaHordeEnabledMode::EnabledForBuildMachineOnly: return TEXT("BuildMachineOnly");
	}
	return TEXT("");
}

UBACOORDINATORHORDE_API bool LexFromString(EUbaHordeEnabledMode& OutEnabledMode, const TCHAR* InString)
{
	if (FCString::Stricmp(InString, TEXT("True")) == 0)
	{
		OutEnabledMode = EUbaHordeEnabledMode::Enabled;
	}
	else if (FCString::Stricmp(InString, TEXT("False")) == 0)
	{
		OutEnabledMode = EUbaHordeEnabledMode::Disabled;
	}
	else if (FCString::Stricmp(InString, TEXT("BuildMachineOnly")) == 0)
	{
		OutEnabledMode = EUbaHordeEnabledMode::EnabledForBuildMachineOnly;
	}
	else
	{
		return false;
	}
	return true;
}


