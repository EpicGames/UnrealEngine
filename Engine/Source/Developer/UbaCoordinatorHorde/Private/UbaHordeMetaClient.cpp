// Copyright Epic Games, Inc. All Rights Reserved.

#include "UbaHordeMetaClient.h"
#include "UbaHordeConfig.h"
#include "Horde.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeExit.h"
#include "HttpServerConstants.h"

DEFINE_LOG_CATEGORY(LogUbaHorde);

bool FUbaHordeMetaClient::RefreshHttpClient()
{
	FString ServerUrlConfigSource;
	if (FHorde::GetServerUrl(ServerUrl, &ServerUrlConfigSource))
	{
		UE_LOGF(LogUbaHorde, Display, "Getting Horde server URL succeeded [URL: %ls, Source: %ls]", *ServerUrl, *ServerUrlConfigSource);
	}
	else
	{
		UE_LOGF(LogUbaHorde, Warning, "Getting Horde server URL failed [Source: %ls]", *ServerUrlConfigSource);
		return false;
	}

	// Try to connect to Horde with HTTP and v2 API
	HttpClient = MakeUnique<FHordeHttpClient>(ServerUrl);

	if (!HttpClient->Login(FApp::IsUnattended()))
	{
		UE_LOGF(LogUbaHorde, Warning, "Login to Horde server [URL: %ls, Source: %ls] failed", *ServerUrl, *ServerUrlConfigSource);
		return false;
	}

	bClientNeedsRefresh.store(false);

	return true;
}

FString FUbaHordeMetaClient::BuildHordeRequestJsonBody(const FString& PoolId, EUbaHordeConnectionMode ConnectionMode, EUbaHordeEncryption Encryption, const TCHAR* ConditionSuffix, bool bExclusiveAccess, bool bAllowWine)
{
	// Construct JSON body for Horde request, e.g. '{ "connection": { "modePreference": "relay", "encryption": "aes" } }'
	const TSharedPtr<FJsonObject> RequestJNode = MakeShared<FJsonObject>();
	{
		if (!PoolId.IsEmpty() || bExclusiveAccess)
		{
			const TSharedPtr<FJsonObject> RequirementsJNode = MakeShared<FJsonObject>();
			{
				if (!PoolId.IsEmpty())
				{
					RequirementsJNode->SetStringField(TEXT("pool"), PoolId);
				}

				FString Condition;
#if PLATFORM_WINDOWS
				Condition.Appendf(TEXT("(OSFamily == 'Windows' || WineEnabled == '%s')"), bAllowWine ? TEXT("true") : TEXT("false"));
#elif PLATFORM_MAC
				Condition.Append(TEXT("OSFamily == 'MacOS'"));
#elif PLATFORM_LINUX
				Condition.Append(TEXT("OSFamily == 'Linux'"));
#endif
				if (ConditionSuffix && *ConditionSuffix != TEXT('\0'))
				{
					Condition.Appendf(TEXT(" %s"), ConditionSuffix);
				}
				RequirementsJNode->SetStringField(TEXT("condition"), Condition);

				RequirementsJNode->SetBoolField(TEXT("exclusive"), bExclusiveAccess);
			}
			RequestJNode->SetObjectField(TEXT("requirements"), RequirementsJNode);
		}

		const TSharedPtr<FJsonObject> ConnectionJNode = MakeShared<FJsonObject>();
		{
			ConnectionJNode->SetStringField(TEXT("modePreference"), LexToString(ConnectionMode));
			if (Encryption != EUbaHordeEncryption::None)
			{
				ConnectionJNode->SetStringField(TEXT("encryption"), LexToString(Encryption));
			}

			const TSharedPtr<FJsonObject> PortsJNode = MakeShared<FJsonObject>();
			{
				PortsJNode->SetNumberField(TEXT("UbaPort"), (double)FHordeRemoteMachineInfo::UbaPort);
				PortsJNode->SetNumberField(TEXT("UbaProxyPort"), (double)FHordeRemoteMachineInfo::UbaProxyPort);
			}
			ConnectionJNode->SetObjectField(TEXT("ports"), PortsJNode);
		}
		RequestJNode->SetObjectField(TEXT("connection"), ConnectionJNode);
	}

	FString JsonContent;
	const TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&JsonContent);
	FJsonSerializer::Serialize(RequestJNode.ToSharedRef(), JsonWriter);
	return JsonContent;
}

TSharedPtr<FUbaHordeMetaClient::HordeClusterPromise, ESPMode::ThreadSafe> FUbaHordeMetaClient::RequestClusterId(const FString& HordeRequestJsonBody)
{
	TSharedPtr<HordeClusterPromise> Promise = MakeShared<HordeClusterPromise, ESPMode::ThreadSafe>();

	if (bClientNeedsRefresh)
	{
		RefreshHttpClient();
	}

	FHttpRequestRef Request = HttpClient->CreateRequest(TEXT("POST"), TEXT("api/v2/compute/_cluster"));

	Request->SetDelegateThreadPolicy(EHttpRequestDelegateThreadPolicy::CompleteOnHttpThread);
	Request->SetContentAsString(HordeRequestJsonBody);
	Request->SetHeader(TEXT("Content-Type"), TEXT("application/json"));

	Request->OnProcessRequestComplete().BindLambda(
		[this, Promise](FHttpRequestPtr /*Request*/, FHttpResponsePtr HttpResponse, bool bConnectedSuccessfully)
		{
			FHordeClusterInfo Info;

			ON_SCOPE_EXIT
			{
				Promise->SetValue(MakeTuple(HttpResponse, Info));
			};

			if (!bConnectedSuccessfully || !HttpResponse.IsValid())
			{
				UE_LOGF(LogUbaHorde, Display, "No response from Horde");
				return;
			}

			const EHttpServerResponseCodes ResponseCode = (EHttpServerResponseCodes)HttpResponse->GetResponseCode();
			const FString ResponseStr = HttpResponse->GetContentAsString();

			if (ResponseCode == EHttpServerResponseCodes::ServiceUnavail ||
				ResponseCode == EHttpServerResponseCodes::TooManyRequests)
			{
				UE_LOGF(LogUbaHorde, Verbose, "Horde agent request returned with HTTP/%d: %ls", (int32)ResponseCode, *ResponseStr);
				return;
			}

			if (ResponseCode == EHttpServerResponseCodes::Denied)
			{
				UE_LOGF(LogUbaHorde, Display, "Token expired, refreshing");
				bClientNeedsRefresh.store(true);
				return;
			}

			TSharedPtr<FJsonValue> OutJson;
			TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(ResponseStr);

			if (!FJsonSerializer::Deserialize(Reader, OutJson, FJsonSerializer::EFlags::None))
			{
				// Report invalid response body with Display verbosity only, since this should not fail a CIS job
				UE_LOGF(LogUbaHorde, Display, "Invalid response body for cluster Id resolution (HTTP/%d): %ls", (int32)ResponseCode, *ResponseStr);
				return;
			}

			TSharedPtr<FJsonValue> ClusterIdValue = OutJson->AsObject()->TryGetField(TEXT("clusterId"));
			if (!ClusterIdValue.Get())
			{
				// Report invalid response body with Display verbosity only, since this should not fail a CIS job
				UE_LOGF(LogUbaHorde, Display, "Missing \"clusterId\" entry in response body (HTTP/%d): %ls", (int32)ResponseCode, *ResponseStr);
				return;
			}

			// Successfully return resolved cluster ID
			Info.ClusterId = ClusterIdValue->AsString();
		});

	Request->ProcessRequest();

	return Promise;
}

TSharedPtr<FUbaHordeMetaClient::HordeMachinePromise, ESPMode::ThreadSafe> FUbaHordeMetaClient::RequestMachine(const FString& HordeRequestJsonBody, const TCHAR* ClusterId)
{
	TSharedPtr<HordeMachinePromise> Promise = MakeShared<HordeMachinePromise, ESPMode::ThreadSafe>();

	if (bClientNeedsRefresh)
	{
		RefreshHttpClient();
	}

	const FString ResourcePath = FString::Format(TEXT("api/v2/compute/{0}"), { ClusterId != nullptr && *ClusterId != TEXT('\0') ? ClusterId : TEXT("default")});
	FHttpRequestRef Request = HttpClient->CreateRequest(TEXT("POST"), *ResourcePath);

	Request->SetDelegateThreadPolicy(EHttpRequestDelegateThreadPolicy::CompleteOnHttpThread);
	Request->SetContentAsString(HordeRequestJsonBody);
	Request->SetHeader(TEXT("Content-Type"), TEXT("application/json"));

	Request->OnProcessRequestComplete().BindLambda(
		[this, Promise, ClusterId = FString(ClusterId)](FHttpRequestPtr /*Request*/, FHttpResponsePtr HttpResponse, bool bConnectedSuccessfully)
		{
			FHordeRemoteMachineInfo Info;

			ON_SCOPE_EXIT
			{
				Promise->SetValue(MakeTuple(HttpResponse, Info));
			};

			Info.Ip = TEXT("");
			Info.Port = 0xFFFF;
			Info.bRunsWindowOS = false;
			FMemory::Memset(Info.Nonce, 0, sizeof(Info.Nonce));

			if (!bConnectedSuccessfully || !HttpResponse.IsValid())
			{
				UE_LOGF(LogUbaHorde, Display, "No response from Horde");
				return;
			}

			const EHttpServerResponseCodes ResponseCode = (EHttpServerResponseCodes)HttpResponse->GetResponseCode();
			const FString ResponseStr = HttpResponse->GetContentAsString();

			if (ResponseCode == EHttpServerResponseCodes::NotFound ||
				ResponseCode == EHttpServerResponseCodes::ServiceUnavail ||
				ResponseCode == EHttpServerResponseCodes::TooManyRequests)
			{
				UE_LOGF(LogUbaHorde, Verbose, "Horde agent request returned with HTTP/%d: %ls", EnumToUnderlyingType(ResponseCode), *ResponseStr);
				return;
			}

			if (ResponseCode == EHttpServerResponseCodes::Denied)
			{
				UE_LOGF(LogUbaHorde, Display, "Token expired, refreshing");
				bClientNeedsRefresh.store(true);
				return;
			}

			TSharedPtr<FJsonValue> OutJson;
			TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(ResponseStr);

			if (!FJsonSerializer::Deserialize(Reader, OutJson, FJsonSerializer::EFlags::None))
			{
				// Report invalid response body with Display verbosity only, since this should not fail a CIS job
				UE_LOGF(LogUbaHorde, Display, "Invalid response body for remote helper request (HTTP/%d): %ls", (int32)ResponseCode, *ResponseStr);
				return;
			}

			TSharedPtr<FJsonValue> NonceValue = OutJson->AsObject()->TryGetField(TEXT("nonce"));
			TSharedPtr<FJsonValue> IpValue = OutJson->AsObject()->TryGetField(TEXT("ip"));
			TSharedPtr<FJsonValue> PortValue = OutJson->AsObject()->TryGetField(TEXT("port"));

			if (!NonceValue.Get() || !IpValue.Get() || !PortValue.Get())
			{
				// Report invalid response body with Display verbosity only, since this should not fail a CIS job
				UE_LOGF(LogUbaHorde, Display, "Missing \"nonce\", \"ip\", or \"port\" entry in response body (HTTP/%d): %ls", (int32)ResponseCode, *ResponseStr);
				return;
			}

			// Check for optional port mapping array
			if (TSharedPtr<FJsonValue> PortsValue = OutJson->AsObject()->TryGetField(TEXT("ports")))
			{
				for (auto Iter = PortsValue->AsObject()->Values.CreateConstIterator(); Iter; ++Iter)
				{
					FHordeRemoteMachineInfo::FPortInfo& PortInfo = Info.Ports.Add(FString(Iter.Key()));
					if (TSharedPtr<FJsonValue> PortInfoPortValue = Iter.Value()->AsObject()->TryGetField(TEXT("port")))
					{
						PortInfo.Port = (uint16)PortInfoPortValue->AsNumber();
					}
					if (TSharedPtr<FJsonValue> PortInfoAgentPortValue = Iter.Value()->AsObject()->TryGetField(TEXT("agentPort")))
					{
						PortInfo.AgentPort = (uint16)PortInfoAgentPortValue->AsNumber();
					}
				}
			}

			// Check for connection mode and address - "connectionAddress" field is meaningless without a mode other than the default one
			if (TSharedPtr<FJsonValue> ConnectionModeValue = OutJson->AsObject()->TryGetField(TEXT("connectionMode")))
			{
				if (LexFromString(Info.ConnectionMode, *ConnectionModeValue->AsString()))
				{
					if (TSharedPtr<FJsonValue> ConnectionAddressValue = OutJson->AsObject()->TryGetField(TEXT("connectionAddress")))
					{
						Info.ConnectionAddress = ConnectionAddressValue->AsString();
					}
				}
			}

			FString OsFamily(TEXT("UNKNOWN-OS"));

			uint16 LogicalCores = 0;
			uint16 PhysicalCores = 0;

			if (TSharedPtr<FJsonValue> PropertiesValue = OutJson->AsObject()->TryGetField(TEXT("properties")))
			{
				for (const TSharedPtr<FJsonValue>& PropertyEntryValue : PropertiesValue->AsArray())
				{
					checkf(PropertyEntryValue.Get(), TEXT("null pointer in JSON array object of node \"properties\""));
					const FString PropertyElementString = PropertyEntryValue->AsString();
					if (PropertyElementString.StartsWith(TEXT("OSFamily=")))
					{
						OsFamily = *PropertyElementString + 9;
						if (OsFamily == TEXT("Windows"))
						{
							Info.bRunsWindowOS = true;
						}
					}
					if (PropertyElementString.StartsWith(TEXT("LogicalCores=")))
					{
						LogicalCores = (uint16)FCString::Atoi(*PropertyElementString + 13);
					}
					if (PropertyElementString.StartsWith(TEXT("PhysicalCores=")))
					{
						PhysicalCores = (uint16)FCString::Atoi(*PropertyElementString + 14);
					}
				}
			}

			if (LogicalCores)
			{
				Info.LogicalCores = LogicalCores;
			}
			else if (PhysicalCores)
			{
				Info.LogicalCores = PhysicalCores * 2;
			}
			else
			{
				Info.LogicalCores = 16; // Wild guess
			}

			// Log summary of assigned Horde machine
			FString NonceString = NonceValue->AsString();
			FString IpString = IpValue->AsString();
			uint16 PortNumber = (uint16)PortValue->AsNumber();

			// Return final response information
			Info.Ip = IpString;
			Info.Port = PortNumber;
			FString::ToHexBlob(NonceString, Info.Nonce, HORDE_NONCE_SIZE);

			TStringBuilder<512> HordeMachineSummary;
			HordeMachineSummary.Appendf(TEXT("UBA Horde machine assigned (%s) on '%s' [%s:%u]"), *OsFamily, *ClusterId, *Info.GetConnectionAddress(), (uint32)Info.GetConnectionPort().Port);

#if WITH_SSL
			if (TSharedPtr<FJsonValue> EncryptionValue = OutJson->AsObject()->TryGetField(TEXT("encryption")))
			{
				if (LexFromString(Info.Encryption, *EncryptionValue->AsString()))
				{
					if (Info.Encryption == EUbaHordeEncryption::AES)
					{
						TSharedPtr<FJsonValue> KeyValue = OutJson->AsObject()->TryGetField(TEXT("key"));
						FString KeyString = KeyValue ? KeyValue->AsString() : FString();
						if (KeyString.IsEmpty())
						{
							HordeMachineSummary << TEXT(" [AES key missing]");
						}
						else if (KeyString.Len() != sizeof(Info.Key) * 2)
						{
							HordeMachineSummary << TEXT(" [AES key corrupted]");
						}
						else
						{
							FString::ToHexBlob(KeyString, Info.Key, sizeof(Info.Key));
							HordeMachineSummary << TEXT(" [AES key received]");
						}
					}
				}
			}
#endif

			if (TSharedPtr<FJsonValue> LeaseIdValue = OutJson->AsObject()->TryGetField(TEXT("leaseId")))
			{
				Info.LeaseLink = FString::Format(TEXT("{0}lease/{1}"), { this->ServerUrl, LeaseIdValue->AsString() });
				HordeMachineSummary << TEXT(": ") << Info.LeaseLink;
			}

			UE_LOGF(LogUbaHorde, Display, "%ls", HordeMachineSummary.ToString());
		});

	Request->ProcessRequest();

	return Promise;
}

TSharedPtr<FUbaHordeMetaClient::HordeCacheServerPromise, ESPMode::ThreadSafe> FUbaHordeMetaClient::RequestCacheServer(const TCHAR* ClusterId)
{
	TSharedPtr<HordeCacheServerPromise> Promise = MakeShared<HordeCacheServerPromise, ESPMode::ThreadSafe>();

	if (bClientNeedsRefresh)
	{
		RefreshHttpClient();
	}

	const FString ResourcePath = FString::Format(TEXT("api/v2/compute/{0}/uba-cache"), { ClusterId != nullptr && *ClusterId != TEXT('\0') ? ClusterId : TEXT("default") });
	FHttpRequestRef Request = HttpClient->CreateRequest(TEXT("POST"), *ResourcePath);

	Request->SetDelegateThreadPolicy(EHttpRequestDelegateThreadPolicy::CompleteOnHttpThread);
	Request->SetContentAsString(TEXT("{}"));
	Request->SetHeader(TEXT("Content-Type"), TEXT("application/json"));

	Request->OnProcessRequestComplete().BindLambda(
		[Promise](FHttpRequestPtr /*Request*/, FHttpResponsePtr HttpResponse, bool bConnectedSuccessfully)
		{
			FHordeCacheServerInfo Info;

			ON_SCOPE_EXIT
			{
				Promise->SetValue(MakeTuple(HttpResponse, Info));
			};

			if (!bConnectedSuccessfully || !HttpResponse.IsValid())
			{
				UE_LOGF(LogUbaHorde, Display, "No response from Horde for cache server request");
				return;
			}

			const EHttpServerResponseCodes ResponseCode = (EHttpServerResponseCodes)HttpResponse->GetResponseCode();
			const FString ResponseStr = HttpResponse->GetContentAsString();

			if (ResponseCode == EHttpServerResponseCodes::ServiceUnavail || ResponseCode == EHttpServerResponseCodes::TooManyRequests)
			{
				UE_LOGF(LogUbaHorde, Verbose, "Horde cache server request returned with HTTP/%d: %ls", (int32)ResponseCode, *ResponseStr);
				return;
			}

			if (ResponseCode == EHttpServerResponseCodes::Denied)
			{
				UE_LOGF(LogUbaHorde, Display, "Token expired during cache server request");
				return;
			}

			TSharedPtr<FJsonValue> OutJson;
			TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(ResponseStr);

			if (!FJsonSerializer::Deserialize(Reader, OutJson, FJsonSerializer::EFlags::None))
			{
				UE_LOGF(LogUbaHorde, Display, "Invalid response body for cache server request (HTTP/%d): %ls", (int32)ResponseCode, *ResponseStr);
				return;
			}

			TSharedPtr<FJsonValue> CacheEndpointValue = OutJson->AsObject()->TryGetField(TEXT("cacheEndpoint"));
			if (!CacheEndpointValue.Get())
			{
				UE_LOGF(LogUbaHorde, Display, "Missing \"cacheEndpoint\" in cache server response (HTTP/%d): %ls", (int32)ResponseCode, *ResponseStr);
				return;
			}

			Info.CacheEndpoint = CacheEndpointValue->AsString();

			if (TSharedPtr<FJsonValue> KeyValue = OutJson->AsObject()->TryGetField(TEXT("cacheSessionKey")))
			{
				Info.CacheSessionKey = KeyValue->AsString();
			}

			if (TSharedPtr<FJsonValue> WriteAccessValue = OutJson->AsObject()->TryGetField(TEXT("writeAccess")))
			{
				Info.bWriteAccess = WriteAccessValue->AsBool();
			}

			UE_LOGF(LogUbaHorde, Display, "Horde cache server assigned: %ls (WriteAccess: %ls)", *Info.CacheEndpoint, Info.bWriteAccess ? TEXT("true") : TEXT("false"));
		});

	Request->ProcessRequest();

	return Promise;
}
