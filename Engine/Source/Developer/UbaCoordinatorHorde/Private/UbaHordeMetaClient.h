// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HttpModule.h"
#include "Async/Future.h"
#include "Containers/UnrealString.h"
#include "Interfaces/IHttpRequest.h"
#include "Interfaces/IHttpResponse.h"
#include "HordeHttpClient.h"
#include "UbaHordeAgentManager.h"
#include "UbaHordeConfig.h"

DECLARE_LOG_CATEGORY_EXTERN(LogUbaHorde, Log, All);

#define HORDE_NONCE_SIZE 64
#define HORDE_KEY_SIZE 32

// When asking the Horde server for a machine, this will be returned when the HTTP
// reponse comes back (after a machine request). If the response didn't come through
// (i.e., Horde was unable to assign a machine or something), then:
// - Ip will be set to ""
// - Port will be set to 0xFFFF
// - Nonce will be all 0
struct FHordeRemoteMachineInfo
{
	struct FPortInfo
	{
		FPortInfo() = default;
		FPortInfo(uint16 InPort, uint16 InAgentPort)
			: Port(InPort)
			, AgentPort(InAgentPort)
		{
		}

		uint16 Port = 0;
		uint16 AgentPort = 0;
	};

	FString Ip;
	EUbaHordeConnectionMode ConnectionMode = EUbaHordeConnectionMode::Direct;
	FString ConnectionAddress; // Onyl used for relay mode
	uint16 Port = 0;
	TMap<FString, FPortInfo> Ports; // Maps compute service names to port/agent-port pairs, e.g. "_horde_compute" -> {Port: 12345, AgentPort: 7000}
	uint16 LogicalCores = 0;
	EUbaHordeEncryption Encryption = EUbaHordeEncryption::None;
	uint8 Nonce[HORDE_NONCE_SIZE] = {};
	uint8 Key[HORDE_KEY_SIZE] = {};
	TArray<uint8> CertificateData; // X.509 certificate for SSL connections in Pkcs12 format.
	bool bRunsWindowOS = false;
	FString LeaseLink;

	static constexpr uint16 UbaPort = 7001;
	static constexpr uint16 UbaProxyPort = 7002;

	const FString& GetConnectionAddress() const
	{
		return ConnectionMode == EUbaHordeConnectionMode::Relay ? ConnectionAddress : Ip;
	}

	FPortInfo GetConnectionPort() const
	{
		static const FString HordeComputeIdentifier = TEXT("_horde_compute");
		return ConnectionMode == EUbaHordeConnectionMode::Relay ? Ports[HordeComputeIdentifier] : FPortInfo(Port, Port);
	}
};

struct FHordeClusterInfo
{
	FString ClusterId = TEXT("default");
};

struct FHordeCacheServerInfo
{
	FString CacheEndpoint;    // "ip:port" format, empty if unavailable
	FString CacheSessionKey;  // AES encryption key for cache communication
	bool bWriteAccess = false;
};

// This encapsulates the mechanism of talking to the Horde "meta server". The server which
// can grant us access to remote machines.
class FUbaHordeMetaClient final
{
public:
	// We return the HttpResponse in case more information could be used out of it later.
	using HordeMachinePromise = TPromise<TTuple<FHttpResponsePtr, FHordeRemoteMachineInfo>>;
	using HordeClusterPromise = TPromise<TTuple<FHttpResponsePtr, FHordeClusterInfo>>;
	using HordeCacheServerPromise = TPromise<TTuple<FHttpResponsePtr, FHordeCacheServerInfo>>;

	FUbaHordeMetaClient() = default;

	bool RefreshHttpClient();

	static FString BuildHordeRequestJsonBody(const FString& PoolId, EUbaHordeConnectionMode ConnectionMode, EUbaHordeEncryption Encryption, const TCHAR* ConditionSuffix = nullptr, bool bExclusiveAccess = true, bool bAllowWine = true);

	TSharedPtr<HordeClusterPromise, ESPMode::ThreadSafe> RequestClusterId(const FString& HordeRequestJsonBody);

	// This will make a request to Horde for a remote machine to do work on.
	// Example of actually getting the FHordeRemoteMachineInfo struct:
	// ```
	// auto Promise = HordeServer.RequestMachine();
	// auto Future = Promise->GetFuture();
	// ... // Can to asynchronous work
	// auto Future.Wait(); // Wait for the response to arrive
	// FHordeRemoteMachineInfo MachineInfo = Future.Get().Value;
	TSharedPtr<HordeMachinePromise, ESPMode::ThreadSafe> RequestMachine(const FString& HordeRequestJsonBody, const TCHAR* ClusterId = nullptr);

	TSharedPtr<HordeCacheServerPromise, ESPMode::ThreadSafe> RequestCacheServer(const TCHAR* ClusterId = nullptr);

private:
	FString ServerUrl;
	TUniquePtr<FHordeHttpClient> HttpClient;
	std::atomic<bool> bClientNeedsRefresh{ true };
};
