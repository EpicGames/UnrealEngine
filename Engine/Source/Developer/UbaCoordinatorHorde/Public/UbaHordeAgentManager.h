// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/AnsiString.h"
#include "Containers/Array.h"
#include "HAL/Thread.h"
#include "UbaBase.h"
#include "UbaHordeConfig.h"

class FEvent;
class FUbaHordeMetaClient;
namespace uba { class NetworkServer; }

class FUbaHordeAgentManager
{
public:
	UBACOORDINATORHORDE_API FUbaHordeAgentManager(const FString& InWorkingDir, const FString& BinariesPath);
	UBACOORDINATORHORDE_API ~FUbaHordeAgentManager();

	UBACOORDINATORHORDE_API void SetTargetCoreCount(uint32 Count);

	using FAddClientCallback = bool(void* UserData, const uba::tchar* Ip, uint16 Port, const uba::tchar* Crypto16Characters);
	UBACOORDINATORHORDE_API void SetAddClientCallback(FAddClientCallback* Callback, void* UserData);

	using FUpdateStatusCallback = void(void* UserData, const TCHAR* Status);
	UBACOORDINATORHORDE_API void SetUpdateStatusCallback(FUpdateStatusCallback* Callback, void* UserData);

	// Returns the number of agents currently handled by this agent manager.
	UBACOORDINATORHORDE_API int32 GetAgentCount() const;

	// Returns the active number of cores allocated across all agents.
	UBACOORDINATORHORDE_API uint32 GetActiveCoreCount() const;

	// Requests a UBA cache server from Horde. Returns true if a cache server was assigned.
	// OutCacheEndpoint will be in "ip:port" format. OutCacheSessionKey is the AES encryption key.
	UBACOORDINATORHORDE_API bool RequestCacheServer(FString& OutCacheEndpoint, FString& OutCacheSessionKey, bool& bOutWriteAccess);

private:
	struct FHordeAgentWrapper
	{
		FThread Thread;
		FEvent* ShouldExit;
	};

	void RequestAgent();
	void ThreadAgent(FHordeAgentWrapper& Wrapper);
	void UpdateStatus(const TCHAR* Status);

	FString WorkingDir;
	FString BinariesPath;

	TUniquePtr<FUbaHordeMetaClient> HordeMetaClient;

	FCriticalSection BundleRefPathsLock;
	TArray<FString> BundleRefPaths;

	mutable FCriticalSection AgentsLock;
	TArray<TUniquePtr<FHordeAgentWrapper>> Agents;

	TAtomic<uint64> LastRequestFailTime;
	TAtomic<uint32> TargetCoreCount;
	TAtomic<uint32> EstimatedCoreCount;
	TAtomic<uint32> ActiveCoreCount;
	TAtomic<uint32> AgentsActive;
	TAtomic<uint32> AgentsRequesting;
	TAtomic<uint32> AgentsInProgress;
	TAtomic<bool> bAskForAgents;

	FAddClientCallback* AddClientCallback = nullptr;
	void* AddClientUserData = nullptr;

	FCriticalSection UpdateStatusLock;
	FString UpdateStatusText;
	uint32 UpdateStatusAgentsInProgress = 0;
	uint32 UpdateStatusAgentsActive = 0;
	uint32 UpdateStatusAgentsRequesting = 0;
	bool UpdateHadFailTime = false;
	FUpdateStatusCallback* UpdateStatusCallback = nullptr;
	void* UpdateStatusUserData = nullptr;

	FUbaHordeAgentManager(const FUbaHordeAgentManager&) = delete;
	void operator=(const FUbaHordeAgentManager&) = delete;
};
