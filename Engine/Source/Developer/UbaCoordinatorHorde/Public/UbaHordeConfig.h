// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "Containers/Array.h"

class FEvent;
class FUbaHordeMetaClient;
namespace uba { class NetworkServer; }

enum class EUbaHordeEncryption
{
	None,
	AES,
};

enum class EUbaHordeConnectionMode
{
	Direct,
	Tunnel,
	Relay,
};

enum class EUbaHordeEnabledMode
{
	Disabled,
	EnabledForBuildMachineOnly,
	Enabled,
};

/** UBA/Horde configuration structure. This is meant to match the C# version UnrealBuildTool.UnrealBuildAcceleratorHordeConfig. */
class FUbaHordeConfig
{
public:
	static const TCHAR* ClusterDefault;
	static const TCHAR* ClusterAuto;

	/* Selected Horde provider, e.g. "Uba.Provider.Horde" by default */
	FString Provider;

	/** Specifies whether UBA/Horde is enabled for the current session. By default disabled. */
	bool bIsProviderEnabled = false;

	/** Horde pool name. */
	FString HordePool;

	/** URI of the Horde server. */
	FString HordeServer;

	/** Authentication token for the Horde server. */
	FString HordeToken;

	/** Requirements for the Horde agent to assign. */
	FString HordeCondition;

	/** Compute cluster ID to use in Horde. Set to "_auto" to let Horde server resolve a suitable cluster. By default "default". */
	FString HordeCluster = FUbaHordeConfig::ClusterDefault;

	/** Which IP address UBA server should assign to agents. */
	FString HordeHost;

	/** Optional sentry URL to send UBA data to. */
	FString UbaSentryUrl;

	/** Maxmium number of CPU cores allowed to be used by build session. */
	int32 HordeMaxCores = 2048;

	/** Maximum workers allowed to be used by build session. */
	int32 HordeMaxWorkers = 64;

	/** Maximum number of local CPU cores allowed to be used by build session. By default -1, which indicates to use as many CPU cores as the local machine can provide. */
	int32 MaxParallelActions = -1;

	/** Allow use of POSIX/Wine. Only applicable to Horde agents running Linux. Can still be ignored if Wine executable is not set on agent. */
	bool bHordeAllowWine = true;

	/** Specifies how to connect to the remote machine. Relay mode implies encrypted connections. */
	EUbaHordeConnectionMode HordeConnectionMode = EUbaHordeConnectionMode::Direct;

	/** Specifies transport layer encryption. Currently only AES encryption is supported. */
	EUbaHordeEncryption HordeEncryption = EUbaHordeEncryption::None;

	/** Returns the active Horde pool depending on override settings and current desktop platform. */
	UBACOORDINATORHORDE_API const FString& GetHordePool() const;

	/** Returns the global UBA/Horde configuration and initializes it from a list of XML files on the first call. */
	UBACOORDINATORHORDE_API static const FUbaHordeConfig& Get();

	// DEPRECATED

	UE_DEPRECATED(5.6, "Use HordePool instead.")
	FString DefaultHordePool;

	UE_DEPRECATED(5.6, "Use HordePool instead.")
	FString OverrideHordePool;

	UE_DEPRECATED(5.6, "Use HordePool instead. Windows specific pool can be set via [Uba.Provider.Horde]:Pool in Config/Win64/BaseEngine.ini")
	FString WindowsHordePool;

	UE_DEPRECATED(5.6, "Use HordePool instead. Mac specific pool can be set via [Uba.Provider.Horde]:Pool in Config/Mac/BaseEngine.ini")
	FString MacHordePool;

	UE_DEPRECATED(5.6, "Use HordePool instead. Linux specific pool can be set via [Uba.Provider.Horde]:Pool in Config/Linux/BaseEngine.ini")
	FString LinuxHordePool;

	UE_DEPRECATED(5.6, "Use bIsProviderEnabled instead")
	EUbaHordeEnabledMode HordeEnabled = EUbaHordeEnabledMode::Disabled;

	UE_DEPRECATED(5.6, "Use bIsProviderEnabled instead")
	UBACOORDINATORHORDE_API bool IsHordeEnabled() const;

private:
	void Initialize();

};

UBACOORDINATORHORDE_API const TCHAR* LexToString(EUbaHordeConnectionMode InConnectionMode);
UBACOORDINATORHORDE_API bool LexFromString(EUbaHordeConnectionMode& OutConnectionMode, const TCHAR* InString);

UBACOORDINATORHORDE_API const TCHAR* LexToString(EUbaHordeEncryption InEncryption);
UBACOORDINATORHORDE_API bool LexFromString(EUbaHordeEncryption& OutEncryption, const TCHAR* InString);

UBACOORDINATORHORDE_API const TCHAR* LexToString(EUbaHordeEnabledMode InEnabledMode);
UBACOORDINATORHORDE_API bool LexFromString(EUbaHordeEnabledMode& OutEnabledMode, const TCHAR* InString);

