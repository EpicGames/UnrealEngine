// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UbaCoordinatorHorde : ModuleRules
{
	public UbaCoordinatorHorde(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		StaticAnalyzerDisabledCheckers.Clear();

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"ApplicationCore",
			"Core",
			"DesktopPlatform",
			"Horde",
			"HTTP",
			"Json",
			"Sockets",
			"SSL",
			"OpenSSL",
			"XmlParser",
		});

		PrivateIncludePathModuleNames.AddRange(new string[] {
			"HTTPServer",
		});

		string UbaSourcePath = Path.Combine(EngineDirectory, "Source", "Programs", "UnrealBuildAccelerator");
		PublicIncludePaths.Add(Path.Combine(UbaSourcePath, "Core", "Public"));
		PublicIncludePaths.Add(Path.Combine(UbaSourcePath, "Common", "Public"));
	}
}
